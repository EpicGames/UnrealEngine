# BackChannel

BackChannel is an experimetal plugin that allows external tools and apps to query for and push data into a running Unreal session.

The goal is to support 2-3 ubiquitous protocols where implementations can be easily created or found for popular languages. The short term focus is OSC over TCP/IP and JSON REST over HTTP.
