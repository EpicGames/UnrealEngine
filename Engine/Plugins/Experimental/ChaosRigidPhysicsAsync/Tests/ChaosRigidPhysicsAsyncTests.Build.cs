// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ChaosRigidPhysicsAsyncTests : TestModuleRules
{
	static ChaosRigidPhysicsAsyncTests()
	{
		if (InTestMode)
		{
			TestMetadata = new Metadata();
			TestMetadata.TestName = "ChaosRigidPhysicsAsync";
			TestMetadata.TestShortName = "ChaosRigidPhysicsAsync";
			TestMetadata.ReportType = "testdata";
			TestMetadata.SupportedPlatforms.Add(UnrealTargetPlatform.Mac);
			// TODO: These don't currently work. Need to investigate later, but disabling to get something working for now.
			//TestMetadata.SupportedPlatforms.Add(UnrealTargetPlatform.Linux);
			//TestMetadata.SupportedPlatforms.Add(UnrealTargetPlatform.Android);
			//TestMetadata.SupportedPlatforms.Add(UnrealTargetPlatform.IOS);

			string PlatformCompilationArgs;
			foreach (var Platform in UnrealTargetPlatform.GetValidPlatforms())
			{
				if (Platform == UnrealTargetPlatform.Android)
				{
					PlatformCompilationArgs = "-allmodules -architectures=arm64";
				}
				else
				{
					PlatformCompilationArgs = "-allmodules";
				}
				TestMetadata.PlatformCompilationExtraArgs.Add(Platform, PlatformCompilationArgs);
				// Add our test tag to limit tests
				TestMetadata.PlatformTags.Add(Platform, "[@ChaosTests]");
			}

			// Platform-specific tags
			TestMetadata.PlatformTags[UnrealTargetPlatform.Linux] += "~[.]~[Slow]";
			TestMetadata.PlatformTags[UnrealTargetPlatform.Android] += "~[Perf]~[Slow]~[AndroidSkip]";

			// Allow Android run for this test
			// Will remove Android from PlatformsRunUnsupported as more diverse types of tests can run on this platform
			TestMetadata.PlatformsRunUnsupported.Remove(UnrealTargetPlatform.Android);
		}
	}

	public ChaosRigidPhysicsAsyncTests(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
                "Core",
                "CoreUObject",
                "Engine",
				"ChaosRigidPhysicsAsync",
				"ChaosTestHarness",
                "RigidPhysics",
		});

		SetupModulePhysicsSupport(Target);

		bWarningsAsErrors = true;
	}
}