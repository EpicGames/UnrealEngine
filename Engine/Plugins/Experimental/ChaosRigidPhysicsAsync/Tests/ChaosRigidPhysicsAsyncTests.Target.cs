// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedPlatforms(UnrealPlatformClass.All)]
 public class ChaosRigidPhysicsAsyncTestsTarget : TestTargetRules
 {
     public ChaosRigidPhysicsAsyncTestsTarget(TargetInfo Target) : base(Target)
     {
		bool bIsShipping = Target.Configuration == UnrealTargetConfiguration.Shipping;
		bool bIsTest = Target.Configuration == UnrealTargetConfiguration.Test;

		bTestsRequireEngine = true;
		bTestsRequireCoreUObject = true;

		bCompileAgainstEngine = true;
		bCompileAgainstCoreUObject = true;
		bMockEngineDefaults = true;
		bUsePlatformFileStub = true;

		bWithLowLevelTestsOverride = true;

		// Enable the new API
		GlobalDefinitions.Add("UE_RIGIDPHYSICS_API_ENABLED=1");

		// Enable constraint handle debugging in Chaos
		if (!(bIsShipping || bIsTest))
		{
			GlobalDefinitions.Add("CHAOS_CONSTRAINTHANDLE_DEBUG_ENABLED=1");
		}

		GlobalDefinitions.Add("CATCH_CONFIG_ENABLE_BENCHMARKING=1");

		bWarningsAsErrors = true;
     }
 }