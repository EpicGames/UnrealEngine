// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/LowLevelTest/ChaosTestHarness.h"
#include "Chaos/LowLevelTest/ChaosTestScene.h"
#include "ChaosRigidPhysicsAsyncTest.h"

#include "RigidPhysics/Geometry/AnyGeometry.h"
#include "RigidTestUtils.h"

#if UE_RIGIDPHYSICS_API_ENABLED

namespace Chaos::LowLevelTest
{
	using FSphereGeometry = UE::Physics::FSphereGeometry;
	using FBoxGeometry = UE::Physics::FBoxGeometry;
	using FCapsuleGeometry = UE::Physics::FCapsuleGeometry;
	using FConvexGeometry = UE::Physics::FConvexGeometry;
	using FTriangleMeshGeometry = UE::Physics::FTriangleMeshGeometry;
	using FHeightFieldGeometry = UE::Physics::FHeightFieldGeometry;
	using FAnyGeometry = UE::Physics::FAnyGeometry;

	template <typename ExpectedGeometryType, typename ActualGeometryType>
	void CheckGeometry(const ExpectedGeometryType& ExpectedGeometry, const ActualGeometryType& ActualGeometry)
	{
		constexpr bool bIsExpectedType = std::is_same_v<ExpectedGeometryType, ActualGeometryType>;
		CHECK(bIsExpectedType);
		if constexpr (bIsExpectedType)
		{
			CHECK(ExpectedGeometry == ActualGeometry);
		}
	}

	TEST_CASE("AnyGeometry: None", "[Chaos][API][Geometry][unit]")
	{
		// Defaults to the first shape type, which is a default constructed sphere.
		FAnyGeometry AnyGeometry;
		REQUIRE(AnyGeometry.IsA<FSphereGeometry>());
		CHECK(AnyGeometry.Get<FSphereGeometry>() == FSphereGeometry());
	}

	TEST_CASE("AnyGeometry: Box", "[Chaos][API][Geometry][unit]")
	{
		FBoxGeometry Box0(FVector3f(1, 2, 3));
		FBoxGeometry Box1(FVector3f(2, 3, 4));
		FSphereGeometry Sphere0(1);

		SECTION("IsA")
		{
			FAnyGeometry AnyGeometry(Box0);
			CHECK(AnyGeometry.IsA<FBoxGeometry>());
			CHECK(!AnyGeometry.IsA<FSphereGeometry>());
		}
		SECTION("TryGet/Get")
		{
			FAnyGeometry AnyGeometry(Box0);
			REQUIRE(AnyGeometry.TryGet<FBoxGeometry>() != nullptr);
			CHECK(*AnyGeometry.TryGet<FBoxGeometry>() == Box0);
			CHECK(AnyGeometry.Get<FBoxGeometry>() == Box0);
			CHECK(AnyGeometry.TryGet<FSphereGeometry>() == nullptr);
		}
		SECTION("Visit")
		{
			FAnyGeometry AnyGeometry(Box0);
			AnyGeometry.Visit([&Box0]<typename GeometryType>(const GeometryType& Geometry) { CheckGeometry(Box0, Geometry); });
		}
		SECTION("Comparison")
		{
			FAnyGeometry AnyGeometry(Box0);
			CHECK(AnyGeometry == Box0);
			CHECK(AnyGeometry != Box1);
			CHECK(AnyGeometry != Sphere0);
		}
		SECTION("Assignment")
		{
			FAnyGeometry AnyGeometry(Sphere0);
			AnyGeometry = Box0;
			CHECK(AnyGeometry != Sphere0);
			CHECK(AnyGeometry == Box0);
			CHECK(AnyGeometry != Box1);
			AnyGeometry = Box1;
			CHECK(AnyGeometry != Sphere0);
			CHECK(AnyGeometry != Box0);
			CHECK(AnyGeometry == Box1);
		}
	}

	TEST_CASE("AnyGeometry: Sphere", "[Chaos][API][Geometry][unit]")
	{
		FBoxGeometry Box0(FVector3f(1, 2, 3));
		FSphereGeometry Sphere0(1);
		FSphereGeometry Sphere1(2);

		SECTION("IsA")
		{
			FAnyGeometry AnyGeometry(Sphere0);
			CHECK(AnyGeometry.IsA<FSphereGeometry>());
			CHECK(!AnyGeometry.IsA<FBoxGeometry>());
		}
		SECTION("TryGet/Get")
		{
			FAnyGeometry AnyGeometry(Sphere0);
			REQUIRE(AnyGeometry.TryGet<FSphereGeometry>() != nullptr);
			CHECK(*AnyGeometry.TryGet<FSphereGeometry>() == Sphere0);
			CHECK(AnyGeometry.Get<FSphereGeometry>() == Sphere0);
			CHECK(AnyGeometry.TryGet<FBoxGeometry>() == nullptr);
		}
		SECTION("Visit")
		{
			FAnyGeometry AnyGeometry(Sphere0);
			AnyGeometry.Visit([&Sphere0]<typename GeometryType>(const GeometryType& Geometry) { CheckGeometry(Sphere0, Geometry); });
		}
		SECTION("Comparison")
		{
			FAnyGeometry AnyGeometry(Sphere0);
			CHECK(AnyGeometry == Sphere0);
			CHECK(AnyGeometry != Sphere1);
			CHECK(AnyGeometry != Box0);
		}
		SECTION("Assignment")
		{
			FAnyGeometry AnyGeometry(Box0);
			AnyGeometry = Sphere0;
			CHECK(AnyGeometry != Box0);
			CHECK(AnyGeometry == Sphere0);
			CHECK(AnyGeometry != Sphere1);
			AnyGeometry = Sphere1;
			CHECK(AnyGeometry != Box0);
			CHECK(AnyGeometry != Sphere0);
			CHECK(AnyGeometry == Sphere1);
		}
	}

	TEST_CASE("AnyGeometry: Capsule", "[Chaos][API][Geometry][unit]")
	{
		FBoxGeometry Box0(FVector3f(1, 2, 3));
		FCapsuleGeometry Capsule0(1, 3);
		FCapsuleGeometry Capsule1(2, 4);

		SECTION("IsA")
		{
			FAnyGeometry AnyGeometry(Capsule0);
			CHECK(AnyGeometry.IsA<FCapsuleGeometry>());
			CHECK(!AnyGeometry.IsA<FBoxGeometry>());
		}
		SECTION("TryGet/Get")
		{
			FAnyGeometry AnyGeometry(Capsule0);
			REQUIRE(AnyGeometry.TryGet<FCapsuleGeometry>() != nullptr);
			CHECK(*AnyGeometry.TryGet<FCapsuleGeometry>() == Capsule0);
			CHECK(AnyGeometry.Get<FCapsuleGeometry>() == Capsule0);
			CHECK(AnyGeometry.TryGet<FBoxGeometry>() == nullptr);
		}
		SECTION("Visit")
		{
			FAnyGeometry AnyGeometry(Capsule0);
			AnyGeometry.Visit([&Capsule0]<typename GeometryType>(const GeometryType& Geometry) { CheckGeometry(Capsule0, Geometry); });
		}
		SECTION("Comparison")
		{
			FAnyGeometry AnyGeometry(Capsule0);
			CHECK(AnyGeometry == Capsule0);
			CHECK(AnyGeometry != Capsule1);
			CHECK(AnyGeometry != Box0);
		}
		SECTION("Assignment")
		{
			FAnyGeometry AnyGeometry(Box0);
			AnyGeometry = Capsule0;
			CHECK(AnyGeometry != Box0);
			CHECK(AnyGeometry == Capsule0);
			CHECK(AnyGeometry != Capsule1);
			AnyGeometry = Capsule1;
			CHECK(AnyGeometry != Box0);
			CHECK(AnyGeometry != Capsule0);
			CHECK(AnyGeometry == Capsule1);
		}
	}

	TEST_CASE("AnyGeometry: Convex", "[Chaos][API][Geometry][unit]")
	{
		FBoxGeometry Box0(FVector3f(1, 2, 3));
		FConvexGeometry Convex0(UE::Physics::MakeConvexBoxGeometrySetup());

		SECTION("IsA")
		{
			FAnyGeometry AnyGeometry(Convex0);
			CHECK(AnyGeometry.IsA<FConvexGeometry>());
			CHECK(!AnyGeometry.IsA<FBoxGeometry>());
		}
		SECTION("TryGet/Get")
		{
			FAnyGeometry AnyGeometry(Convex0);
			REQUIRE(AnyGeometry.TryGet<FConvexGeometry>() != nullptr);
			CHECK(*AnyGeometry.TryGet<FConvexGeometry>() == Convex0);
			CHECK(AnyGeometry.Get<FConvexGeometry>() == Convex0);
			CHECK(AnyGeometry.TryGet<FBoxGeometry>() == nullptr);
		}
		SECTION("Visit")
		{
			FAnyGeometry AnyGeometry(Convex0);
			AnyGeometry.Visit([&Convex0]<typename GeometryType>(const GeometryType& Geometry) { CheckGeometry(Convex0, Geometry); });
		}
		SECTION("Comparison")
		{
			FAnyGeometry AnyGeometry(Convex0);
			CHECK(AnyGeometry == Convex0);
			CHECK(AnyGeometry != Box0);
		}
		SECTION("Assignment")
		{
			FAnyGeometry AnyGeometry(Box0);
			AnyGeometry = Convex0;
			CHECK(AnyGeometry != Box0);
			CHECK(AnyGeometry == Convex0);
		}
	}

	TEST_CASE("AnyGeometry: TriangleMesh", "[Chaos][API][Geometry][unit]")
	{
		FBoxGeometry Box0(FVector3f(1, 2, 3));
		FTriangleMeshGeometry TriMesh0(UE::Physics::MakeQuadTriangleGeometrySetup());

		SECTION("IsA")
		{
			FAnyGeometry AnyGeometry(TriMesh0);
			CHECK(AnyGeometry.IsA<FTriangleMeshGeometry>());
			CHECK(!AnyGeometry.IsA<FBoxGeometry>());
		}
		SECTION("TryGet/Get")
		{
			FAnyGeometry AnyGeometry(TriMesh0);
			REQUIRE(AnyGeometry.TryGet<FTriangleMeshGeometry>() != nullptr);
			CHECK(*AnyGeometry.TryGet<FTriangleMeshGeometry>() == TriMesh0);
			CHECK(AnyGeometry.Get<FTriangleMeshGeometry>() == TriMesh0);
			CHECK(AnyGeometry.TryGet<FBoxGeometry>() == nullptr);
		}
		SECTION("Visit")
		{
			FAnyGeometry AnyGeometry(TriMesh0);
			AnyGeometry.Visit([&TriMesh0]<typename GeometryType>(const GeometryType& Geometry) { CheckGeometry(TriMesh0, Geometry); });
		}
		SECTION("Comparison")
		{
			FAnyGeometry AnyGeometry(TriMesh0);
			CHECK(AnyGeometry == TriMesh0);
			CHECK(AnyGeometry != Box0);
		}
		SECTION("Assignment")
		{
			FAnyGeometry AnyGeometry(Box0);
			AnyGeometry = TriMesh0;
			CHECK(AnyGeometry != Box0);
			CHECK(AnyGeometry == TriMesh0);
		}
	}
	TEST_CASE("AnyGeometry: HeightField", "[Chaos][API][Geometry][unit]")
	{
		FBoxGeometry Box0(FVector3f(1, 2, 3));
		FHeightFieldGeometry HeightField0(UE::Physics::MakeHeightFieldGeometrySetup());

		SECTION("IsA")
		{
			FAnyGeometry AnyGeometry(HeightField0);
			CHECK(AnyGeometry.IsA<FHeightFieldGeometry>());
			CHECK(!AnyGeometry.IsA<FBoxGeometry>());
		}
		SECTION("TryGet/Get")
		{
			FAnyGeometry AnyGeometry(HeightField0);
			REQUIRE(AnyGeometry.TryGet<FHeightFieldGeometry>() != nullptr);
			CHECK(*AnyGeometry.TryGet<FHeightFieldGeometry>() == HeightField0);
			CHECK(AnyGeometry.Get<FHeightFieldGeometry>() == HeightField0);
			CHECK(AnyGeometry.TryGet<FBoxGeometry>() == nullptr);
		}
		SECTION("Visit")
		{
			FAnyGeometry AnyGeometry(HeightField0);
			AnyGeometry.Visit([&HeightField0]<typename GeometryType>(const GeometryType& Geometry) { CheckGeometry(HeightField0, Geometry); });
		}
		SECTION("Comparison")
		{
			FAnyGeometry AnyGeometry(HeightField0);
			CHECK(AnyGeometry == HeightField0);
			CHECK(AnyGeometry != Box0);
		}
		SECTION("Assignment")
		{
			FAnyGeometry AnyGeometry(Box0);
			AnyGeometry = HeightField0;
			CHECK(AnyGeometry != Box0);
			CHECK(AnyGeometry == HeightField0);
		}
	}
} // namespace Chaos::LowLevelTest

#endif // UE_RIGIDPHYSICS_API_ENABLED
