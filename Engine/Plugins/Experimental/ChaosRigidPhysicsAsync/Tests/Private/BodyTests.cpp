// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/LowLevelTest/ChaosTestHarness.h"
#include "Chaos/LowLevelTest/ChaosTestScene.h"
#include "ChaosRigidPhysicsAsyncTest.h"
#include "ChaosRigidPhysicsAsync/RigidFactoryAsync.h"
#include "ChaosRigidPhysicsAsync/RigidBodyAsync.h"
#include "ChaosRigidPhysicsAsync/RigidSceneAsync.h"
#include "ChaosRigidPhysicsAsync/RigidSceneSettingsAsync.h"
#include "RigidPhysics/RigidBody.h"
#include "RigidPhysics/RigidContext.h"
#include "RigidPhysics/RigidModifier.h"
#include "RigidPhysics/RigidScene.h"
#include "RigidPhysics/RigidShapeInstance.h"

#include "RigidTestFixture.h"
#include "RigidTestUtils.h"

#if UE_RIGIDPHYSICS_API_ENABLED

namespace Chaos::LowLevelTest
{
	using namespace Chaos::Rigids::Async;
	using namespace UE::Physics;

	TEST_CASE("RigidBody::ShapeInstance::CreateShape", "[Chaos][API][Body][ShapeInstance][unit]")
	{
		FRigidTestFixture Fixture;

		FRigidBodyHandle BodyHandle;
		const TArray<FRigidShapeInstanceSetup> ShapeSetups
		{
			FRigidShapeInstanceSetup(FBoxGeometry(FVector3f(1))),
			FRigidShapeInstanceSetup(FSphereGeometry(2)),
		};
		TArray<FRigidShapeInstanceHandle> ShapeHandles;
		ShapeHandles.SetNum(ShapeSetups.Num());

		auto ValidateGTData = [&Fixture, &BodyHandle, &ShapeSetups, &ShapeHandles](int32 ShapeCount)
			{
				CAPTURE(ShapeCount);
				if (FRigidContextGameRO Context = Fixture.LockROChecked())
				{
					TRigidBodyPtr<FRigidContextGameRO> BodyPtr = BodyHandle.Pin(Context);
					REQUIRE(BodyPtr);
					REQUIRE(BodyPtr->GetNumShapes() == ShapeCount);
					for (int32 I = 0; I < ShapeCount; ++I)
					{
						const TRigidShapeInstancePtr<FRigidContextGameRO> ShapePtr = ShapeHandles[I].Pin(Context);
						REQUIRE(ShapePtr);
						CHECK(BodyPtr->GetShape(I) == ShapePtr);
						CHECK(ShapePtr->GetGeometry() == ShapeSetups[I].Geometry);
					}
				}
			};
		auto ValidatePTData = [&Fixture, &BodyHandle, &ShapeSetups, &ShapeHandles](int32 ShapeCount)
			{
				CAPTURE(ShapeCount);
				Fixture.RunPTCallback([&ShapeSetups, &ShapeHandles, BodyHandle, ShapeCount](const FRigidContextSimRW& Context)
					{
						TRigidBodyPtr<FRigidContextSimRW> BodyPtr = BodyHandle.Pin(Context);
						REQUIRE(BodyPtr);
						REQUIRE(BodyPtr->GetNumShapes() == ShapeCount);
						for (int32 I = 0; I < ShapeCount; ++I)
						{
							const TRigidShapeInstancePtr<FRigidContextSimRW> ShapePtr = ShapeHandles[I].Pin(Context);
							REQUIRE(ShapePtr);
							CHECK(BodyPtr->GetShape(I) == ShapePtr);
							CHECK(ShapePtr->GetGeometry() == ShapeSetups[I].Geometry);
						}
					});
			};

		SECTION("One Shot")
		{
			if (FRigidContextGameRW Context = Fixture.LockRWChecked())
			{
				TRigidBodyPtr<FRigidContextGameRW> BodyPtr = Context->CreateBody(FRigidDebugName("Body1"), ERigidMovementType::Dynamic);
				BodyHandle = BodyPtr;
				Fixture.AutoCleanup(BodyHandle);
				for (int32 I = 0; I < ShapeSetups.Num(); ++I)
				{
					ShapeHandles[I] = BodyPtr->CreateShape(ShapeSetups[I]);
				}
				BodyPtr->Activate();
			}

			ValidateGTData(ShapeSetups.Num());
			ValidatePTData(ShapeSetups.Num());
		}
		SECTION("Incremental")
		{
			if (FRigidContextGameRW Context = Fixture.LockRWChecked())
			{
				TRigidBodyPtr<FRigidContextGameRW> BodyPtr = Context->CreateBody(FRigidDebugName("Body1"), ERigidMovementType::Dynamic);
				BodyHandle = BodyPtr;
				Fixture.AutoCleanup(BodyHandle);
				ShapeHandles[0] = BodyPtr->CreateShape(ShapeSetups[0]);
				BodyPtr->Activate();
			}
			// Check initial config
			ValidateGTData(1);
			ValidatePTData(1);

			// Incrementally add each shape
			for (int32 Index = 1; Index < ShapeHandles.Num(); ++Index)
			{
				if (FRigidContextGameRW Context = Fixture.LockRWChecked())
				{
					const TRigidBodyPtr<FRigidContextGameRW> BodyPtr = BodyHandle.Pin(Context);
					ShapeHandles[Index] = BodyPtr->CreateShape(ShapeSetups[Index]);
				}

				ValidateGTData(Index + 1);
				ValidatePTData(Index + 1);
			}
		}
	}

	TEST_CASE("RigidBody::ShapeInstance::DestroyShape", "[Chaos][API][Body][ShapeInstance][unit]")
	{
		FRigidTestFixture Fixture;

		FRigidBodyHandle BodyHandle;
		const TArray<FRigidShapeInstanceSetup> ShapeSetups
		{
			FRigidShapeInstanceSetup(FBoxGeometry(FVector3f(1))),
			FRigidShapeInstanceSetup(FSphereGeometry(2)),
		};
		TArray<FRigidShapeInstanceHandle> ShapeHandles;
		ShapeHandles.SetNum(ShapeSetups.Num());

		if (FRigidContextGameRW Context = Fixture.LockRWChecked())
		{
			TRigidBodyPtr<FRigidContextGameRW> BodyPtr = Context->CreateBody(FRigidDebugName("Body1"), ERigidMovementType::Dynamic);
			BodyHandle = BodyPtr;
			Fixture.AutoCleanup(BodyHandle);
			for (int32 I = 0; I < ShapeSetups.Num(); ++I)
			{
				ShapeHandles[I] = BodyPtr->CreateShape(ShapeSetups[I]);
			}
			BodyPtr->Activate();
		}

		// Pump the thread to make sure the shapes have been propagated
		Fixture.RunPTCallback([](const FRigidContextSimRW& Context) {});

		SECTION("Remove One Shape")
		{
			const int32 RemovalIndex = GENERATE(0, 1);
			const int32 RemainingIndex = (RemovalIndex + 1) % 2;
			CAPTURE(RemovalIndex);
			if (FRigidContextGameRW Context = Fixture.LockRWChecked())
			{
				TRigidBodyPtr<FRigidContextGameRW> BodyPtr = BodyHandle.Pin(Context);
				BodyPtr->DestroyShape(ShapeHandles[RemovalIndex].Pin(Context));
				REQUIRE(BodyPtr->GetNumShapes() == 1);
				CHECK(BodyPtr->GetShape(0) == ShapeHandles[RemainingIndex].Pin(Context));
				CHECK(!ShapeHandles[RemovalIndex].Pin(Context));
				CHECK(ShapeHandles[RemainingIndex].Pin(Context));
			}
			if (FRigidContextGameRO Context = Fixture.LockROChecked())
			{
				const TRigidBodyPtr<FRigidContextGameRO> BodyPtr = BodyHandle.Pin(Context);
				REQUIRE(BodyPtr->GetNumShapes() == 1);
				CHECK(BodyPtr->GetShape(0) == ShapeHandles[RemainingIndex].Pin(Context));
				CHECK(!ShapeHandles[RemovalIndex].Pin(Context));
				CHECK(ShapeHandles[RemainingIndex].Pin(Context));
			}
			Fixture.RunPTCallback([&ShapeSetups, &ShapeHandles, BodyHandle, RemovalIndex, RemainingIndex](const FRigidContextSimRW& Context)
				{
					const TRigidBodyPtr<FRigidContextSimRW> BodyPtr = BodyHandle.Pin(Context);
					REQUIRE(BodyPtr->GetNumShapes() == 1);
					CHECK(BodyPtr->GetShape(0) == ShapeHandles[RemainingIndex].Pin(Context));
					CHECK(!ShapeHandles[RemovalIndex].Pin(Context));
					CHECK(ShapeHandles[RemainingIndex].Pin(Context));
				});
		}
		SECTION("Remove All Shapes")
		{
			if (FRigidContextGameRW Context = Fixture.LockRWChecked())
			{
				TRigidBodyPtr<FRigidContextGameRW> BodyPtr = BodyHandle.Pin(Context);
				for (int32 I = 0; I < ShapeHandles.Num(); ++I)
				{
					BodyPtr->DestroyShape(ShapeHandles[I].Pin(Context));
					CHECK(!ShapeHandles[I].Pin(Context));
				}
				CHECK(BodyPtr->GetNumShapes() == 0);
			}
			if (FRigidContextGameRO Context = Fixture.LockROChecked())
			{
				CHECK(BodyHandle.Pin(Context)->GetNumShapes() == 0);
				for (int32 I = 0; I < ShapeHandles.Num(); ++I)
				{
					CHECK(!ShapeHandles[I].Pin(Context));
				}
			}
			Fixture.RunPTCallback([&ShapeHandles, BodyHandle](const FRigidContextSimRW& Context)
				{
					CHECK(BodyHandle.Pin(Context)->GetNumShapes() == 0);
					for (int32 I = 0; I < ShapeHandles.Num(); ++I)
					{
						CHECK(!ShapeHandles[I].Pin(Context));
					}
				});
		}
	}

	TEST_CASE("RigidBody::ShapeInstance::Destroy Body With Shapes", "[Chaos][API][Body][ShapeInstance][unit]")
	{
		FRigidTestFixture Fixture;

		FRigidBodyHandle BodyHandle;
		const TArray<FRigidShapeInstanceSetup> ShapeSetups
		{
			FRigidShapeInstanceSetup(FBoxGeometry(FVector3f(1))),
			FRigidShapeInstanceSetup(FSphereGeometry(2)),
		};
		TArray<FRigidShapeInstanceHandle> ShapeHandles;
		ShapeHandles.SetNum(ShapeSetups.Num());

		if (FRigidContextGameRW Context = Fixture.LockRWChecked())
		{
			TRigidBodyPtr<FRigidContextGameRW> BodyPtr = Context->CreateBody(FRigidDebugName("Body1"), ERigidMovementType::Dynamic);
			BodyHandle = BodyPtr;
			for (int32 I = 0; I < ShapeSetups.Num(); ++I)
			{
				ShapeHandles[I] = BodyPtr->CreateShape(ShapeSetups[I]);
			}
			BodyPtr->Activate();
		}

		// Pump the thread to make sure the shapes have been propagated
		Fixture.RunPTCallback([](const FRigidContextSimRW& Context) {});

		if (FRigidContextGameRW Context = Fixture.LockRWChecked())
		{
			Context->DestroyBody(BodyHandle.Pin(Context));

			CHECK(!BodyHandle.Pin(Context).IsValid());
			for (int32 I = 0; I < ShapeHandles.Num(); ++I)
			{
				CHECK(!ShapeHandles[I].Pin(Context).IsValid());
			}
		}
		if (FRigidContextGameRO Context = Fixture.LockROChecked())
		{
			CHECK(!BodyHandle.Pin(Context).IsValid());
			for (int32 I = 0; I < ShapeHandles.Num(); ++I)
			{
				CHECK(!ShapeHandles[I].Pin(Context).IsValid());
			}
		}
		Fixture.RunPTCallback([BodyHandle, &ShapeHandles](const FRigidContextSimRW& Context)
			{
				CHECK(!BodyHandle.Pin(Context).IsValid());
				for (int32 I = 0; I < ShapeHandles.Num(); ++I)
				{
					CHECK(!ShapeHandles[I].Pin(Context).IsValid());
				}
			});
	}

	// Create bodies with various setups and validate them
	TEST_CASE("BodySettingsTest", "[Chaos][API][Body][unit]")
	{
		FRigidFactoryAsync Factory;
		FRigidSceneSettingsAsync SceneSettings;
		FRigidSceneHandle SceneHandle = Factory.CreateScene(FRigidDebugName::Make(TEXT("Main")), &SceneSettings);

		FRigidBodyHandle Body1Handle;
		FRigidBodyHandle Body2Handle;
		if (FRigidContextGameRW Context = SceneHandle.LockRW())
		{
			TRigidBodyPtr<FRigidContextGameRW> Body1Ptr = Context->CreateBody(FRigidDebugName("Body1"), ERigidMovementType::Static);
			REQUIRE(Body1Ptr);
			Body1Ptr->CreateShape(MakeBoxShape(FVector3f(1000, 1000, 100)));
			Body1Ptr->InitTransform(FTransform(FVector(0, 0, -50)));
			Body1Ptr->Activate();

			TRigidBodyPtr<FRigidContextGameRW> Body2Ptr = Context->CreateBody(FRigidDebugName("Body2"), ERigidMovementType::Dynamic);
			REQUIRE(Body2Ptr);
			Body2Ptr->CreateShape(MakeBoxShape(FVector3f(100, 100, 100)));
			Body2Ptr->SetIsSleeping(false);
			Body2Ptr->SetMass(1000);
			Body2Ptr->SetInertia(FVector3d(MakeSolidBoxInertia(1000, FVector3f(100, 100, 100))));
			Body2Ptr->InitTransform(FTransform(FVector(0, 0, 300)));
			Body2Ptr->Activate();

			// Check the bodies are where we think they are and the bounds are up to date
			FBounds3d Body1Bounds = Body1Ptr->GetBounds();
			CHECK(FVector3d(Body1Bounds.Min) == FVector3d(-500, -500, -100));
			CHECK(FVector3d(Body1Bounds.Max) == FVector3d(500, 500, 0));

			FBounds3d Body2Bounds = Body2Ptr->GetBounds();
			CHECK(FVector3d(Body2Bounds.Min) == FVector3d(-50, -50, 250));
			CHECK(FVector3d(Body2Bounds.Max) == FVector3d(50, 50, 350));

			Body1Handle = Body1Ptr;
			Body2Handle = Body2Ptr;
		}

		if (FRigidContextGameRW Context = SceneHandle.LockRW())
		{
			TRigidBodyPtr<FRigidContextGameRW> Body2Ptr = Body2Handle.Pin(Context);
			REQUIRE(Body2Ptr);

			for (int32 TickIndex = 0; TickIndex < 100; ++TickIndex)
			{
				Context->StartTick(0.01f);
				Context->EndTick();
			}

			CHECK_THAT(Body2Ptr->GetTransform().GetLocation(), Catch::ApproxEq(FVec3(0.0, 0.0, 50.0)));
		}

		if (FRigidContextGameRW Context = SceneHandle.LockRW())
		{
			Context->DestroyBody(Body1Handle.Pin(Context));
			Context->DestroyBody(Body2Handle.Pin(Context));
		}

		Factory.DestroyScene(SceneHandle);
	}
}

#endif


#if 0

template<typename InAllocatorType>
void TInitBodiesHelperBase<InAllocatorType>::CreateActor_AssumesLocked(FBodyInstance* Instance, const FTransform& Transform) const
{
	SCOPE_CYCLE_COUNTER(STAT_CreatePhysicsActor);
	checkSlow(!FPhysicsInterface::IsValid(Instance->GetPhysicsActor()));
	const ECollisionEnabled::Type CollisionType = Instance->GetCollisionEnabled();

	FActorCreationParams ActorParams;
	ActorParams.InitialTM = Transform;
	ActorParams.bSimulatePhysics = Instance->ShouldInstanceSimulatingPhysics();
	ActorParams.bStartAwake = Instance->bStartAwake;
#if USE_BODYINSTANCE_DEBUG_NAMES
	ActorParams.DebugName = Instance->CharDebugName.IsValid() ? Instance->CharDebugName->GetData() : nullptr;
#endif
	ActorParams.bEnableGravity = Instance->bEnableGravity;
	ActorParams.bUpdateKinematicFromSimulation = Instance->bUpdateKinematicFromSimulation;
	ActorParams.bQueryOnly = CollisionType == ECollisionEnabled::QueryOnly;
	ActorParams.Scene = PhysScene;
	ActorParams.bStatic = IsStatic();

	FPhysicsActorHandle ActorHandle = nullptr;
	FPhysicsInterface::CreateActor(ActorParams, ActorHandle, PrimitiveComp);
	Instance->SetPhysicsActor(ActorHandle);

	if (!IsStatic())
	{
		FPhysicsInterface::SetCcdEnabled_AssumesLocked(ActorHandle, Instance->bUseCCD);
		FPhysicsInterface::SetMACDEnabled_AssumesLocked(ActorHandle, Instance->IsUsingMACD());
		FPhysicsInterface::SetPartialIslandSleepAllowed_AssumesLocked(ActorHandle, Instance->IsPartialIslandSleepAllowed());
		FPhysicsInterface::SetIsKinematic_AssumesLocked(ActorHandle, !Instance->ShouldInstanceSimulatingPhysics());

		FPhysicsInterface::SetMaxLinearVelocity_AssumesLocked(ActorHandle, TNumericLimits<float>::Max());
		FPhysicsInterface::SetSmoothEdgeCollisionsEnabled_AssumesLocked(ActorHandle, Instance->bSmoothEdgeCollisions);
		FPhysicsInterface::SetInertiaConditioningEnabled_AssumesLocked(ActorHandle, Instance->IsInertiaConditioningEnabled());
		FPhysicsInterface::SetGyroscopicTorqueEnabled_AssumesLocked(ActorHandle, Instance->bGyroscopicTorqueEnabled);
		FPhysicsInterface::SetPositionSolverIterationCount_AssumesLocked(ActorHandle, Instance->GetPositionSolverIterationCount());
		FPhysicsInterface::SetVelocitySolverIterationCount_AssumesLocked(ActorHandle, Instance->GetVelocitySolverIterationCount());
		FPhysicsInterface::SetProjectionSolverIterationCount_AssumesLocked(ActorHandle, Instance->GetProjectionSolverIterationCount());
		FPhysicsInterface::SetGravityGroupIndex_AssumesLocked(ActorHandle, Instance->GravityGroupIndex);

		// Set sleep event notification
		FPhysicsInterface::SetSendsSleepNotifies_AssumesLocked(ActorHandle, Instance->bGenerateWakeEvents);
	}
}

template<typename InAllocatorType>
bool TInitBodiesHelperBase<InAllocatorType>::CreateShapes_AssumesLocked(FBodyInstance* Instance) const
{
	SCOPE_CYCLE_COUNTER(STAT_CreatePhysicsShapes);
	UPhysicalMaterial* SimplePhysMat = Instance->GetSimplePhysicalMaterial();
	TArray<UPhysicalMaterial*> ComplexPhysMats;
	TArray<FPhysicalMaterialMaskParams> ComplexPhysMatMasks;

	ComplexPhysMats = Instance->GetComplexPhysicalMaterials(ComplexPhysMatMasks);

	FBodyCollisionData BodyCollisionData;
	Instance->BuildBodyFilterData(BodyCollisionData.CollisionFilterData);
	FBodyInstance::BuildBodyCollisionFlags(BodyCollisionData.CollisionFlags, Instance->GetCollisionEnabled(), BodySetup->GetCollisionTraceFlag() == CTF_UseComplexAsSimple);

	bool bInitFail = false;

	// #PHYS2 Call interface AddGeometry
	BodySetup->AddShapesToRigidActor_AssumesLocked(Instance, Instance->Scale3D, SimplePhysMat, ComplexPhysMats, ComplexPhysMatMasks, BodyCollisionData, FTransform::Identity);

	FPhysicsInterface::SetIgnoreAnalyticCollisions_AssumesLocked(Instance->GetPhysicsActor(), CVarIgnoreAnalyticCollisionsOverride.GetValueOnAnyThread() ? true : Instance->bIgnoreAnalyticCollisions);

	const int32 NumShapes = FPhysicsInterface::GetNumShapes(Instance->GetPhysicsActor());
	bInitFail |= NumShapes == 0;

	return bInitFail;
}

void UBodySetup::AddShapesToRigidActor_AssumesLocked(
	FBodyInstance* OwningInstance,
	FVector& Scale3D,
	UPhysicalMaterial* SimpleMaterial,
	TArray<UPhysicalMaterial*>& ComplexMaterials,
	TArray<FPhysicalMaterialMaskParams>& ComplexMaterialMasks,
	const FBodyCollisionData& BodyCollisionData,
	const FTransform& RelativeTM,
	TArray<FPhysicsShapeHandle>* NewShapes)
{
	SCOPE_CYCLE_COUNTER(STAT_AddShapesToActor);

	check(OwningInstance);

	// in editor, there are a lot of things relying on body setup to create physics meshes
	CreatePhysicsMeshes();

	// To AddGeometry in interface
	// if almost zero, set min scale
	// @todo fixme
	if (Scale3D.IsNearlyZero())
	{
		// set min scale
		Scale3D = FVector(0.1f);
	}

	FGeometryAddParams AddParams;
	AddParams.bDoubleSided = bDoubleSidedGeometry;
	AddParams.CollisionData = BodyCollisionData;
	AddParams.CollisionTraceType = GetCollisionTraceFlag();
	AddParams.Scale = Scale3D;
	AddParams.SimpleMaterial = SimpleMaterial;
	AddParams.ComplexMaterials = TArrayView<UPhysicalMaterial*>(ComplexMaterials);
	AddParams.ComplexMaterialMasks = TArrayView<FPhysicalMaterialMaskParams>(ComplexMaterialMasks);
	AddParams.LocalTransform = RelativeTM;
	AddParams.WorldTransform = OwningInstance->GetUnrealWorldTransform();
	AddParams.Geometry = &AggGeom;
	AddParams.TriMeshGeometries = MakeArrayView(TriMeshGeometries);

	{
		SCOPE_CYCLE_COUNTER(STAT_AddGeomToSolver);
		FPhysicsInterface::AddGeometry(OwningInstance->GetPhysicsActor(), AddParams, NewShapes);
	}
}

void FPhysInterface_Chaos::AddGeometry(const FPhysicsActorHandle& InActor, const FGeometryAddParams& InParams, TArray<FPhysicsShapeHandle>* OutOptShapes)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPhysInterface_Chaos::AddGeometry);
	LLM_SCOPE(ELLMTag::ChaosGeometry);

	// @todo(chaos): we should not be creating unique geometry per actor
	// @todo(chaos): we are creating the Shapes array twice. Once here and again in SetGeometry or MergeGeometry. Fix this.
	TArray<Chaos::FImplicitObjectPtr> Geoms;
	Chaos::FShapesArray Shapes;
	ChaosInterface::CreateGeometry(InParams, Geoms, Shapes);

	if (InActor && Geoms.Num())
	{
		for (TUniquePtr<Chaos::FPerShapeData>& Shape : Shapes)
		{
			FPhysicsShapeHandle NewHandle(Shape.Get(), InActor);
			if (OutOptShapes)
			{
				OutOptShapes->Add(NewHandle);
			}

			FBodyInstance::ApplyMaterialToShape_AssumesLocked(NewHandle, InParams.SimpleMaterial, InParams.ComplexMaterials, &InParams.ComplexMaterialMasks);

			//TArrayView<UPhysicalMaterial*> SimpleView = MakeArrayView(&(const_cast<UPhysicalMaterial*>(InParams.SimpleMaterial)), 1);
			//FPhysInterface_Chaos::SetMaterials(NewHandle, InParams.ComplexMaterials.Num() > 0 ? InParams.ComplexMaterials : SimpleView);
		}

		// NOTE: Both MergeGeometry and SetGeometry will extend the ShapesInstances array to contain enough elements for
		// each geometry in the Union. However the shape data will not have been filled in, hence the call to MergeShapeInstance at the end.
		// todo: we should not be creating unique geometry per actor
		{
			if (InActor->GetGameThreadAPI().GetGeometry())
			{
				// Geometry already exists - combine new geometry with the existing
				// NOTE: We do not need to set the AllowBVH flag because it will be cloned (see below)
				InActor->GetGameThreadAPI().MergeGeometry(MoveTemp(Geoms));
			}
			else
			{
				// We always have a union so we can support any future welding operations. (Non-trivial converting the SharedPtr to UniquePtr).
				// NOTE: The root union always supports BVH (if there are enough shapes) and is the only Union in the hierarchy that is allowed 
				// to do so, but we don't create it here because that makes welding even more expensive (bodies are welded one by one). 
				// Search for SetAllowBVH to see where the BVH is enabled.
				Chaos::FImplicitObjectPtr Union = MakeImplicitObjectPtr<Chaos::FImplicitObjectUnion>(MoveTemp(Geoms));
				InActor->GetGameThreadAPI().SetGeometry(MoveTemp(Union));
			}
		}

		// Update the newly added shapes with the collision filters, materials etc
		// NOTE: MergeShapes overwrites the last N shapes (see comments above)
		InActor->GetGameThreadAPI().MergeShapesArray(MoveTemp(Shapes));
	}
}

#endif
