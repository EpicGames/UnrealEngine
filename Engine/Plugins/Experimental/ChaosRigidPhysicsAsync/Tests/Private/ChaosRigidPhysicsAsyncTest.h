// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/LowLevelTest/ChaosTestHarness.h"
#include "Chaos/LowLevelTest/ChaosTestScene.h"
#include "RigidPhysics/Geometry/AnyGeometry.h"
#include "RigidPhysics/RigidTyped.h"

#include <string>

namespace Chaos::LowLevelTest
{
	inline std::string ToStdString(const FString& InString)
	{
		return std::string(TCHAR_TO_UTF8(*InString));
	}
}

namespace Catch
{
	// Convert FRigidTypeId to string for output in Catch2 logs
	template<>
	struct StringMaker<UE::Physics::FRigidTypeId>
	{
		static std::string convert(UE::Physics::FRigidTypeId const& Value) { return Chaos::LowLevelTest::ToStdString(Value.GetTypeName()); }
	};
	template<>
	struct StringMaker<UE::Physics::FSphereGeometry>
	{
		static std::string convert(UE::Physics::FSphereGeometry const& Value) { return StringMaker<FString>::convert(Value.ToString()); }
	};
	template<>
	struct StringMaker<UE::Physics::FBoxGeometry>
	{
		static std::string convert(UE::Physics::FBoxGeometry const& Value) { return StringMaker<FString>::convert(Value.ToString()); }
	};
	template<>
	struct StringMaker<UE::Physics::FAnyGeometry>
	{
		static std::string convert(UE::Physics::FAnyGeometry const& Value) { return StringMaker<FString>::convert(Value.ToString()); }
	};
}

