// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/LowLevelTest/ChaosTestHarness.h"
#include "ChaosRigidPhysicsAsyncTest.h"

#include "RigidPhysics/Geometry/ConvexGeometry.h"
#include "RigidTestUtils.h"

#if UE_RIGIDPHYSICS_API_ENABLED

namespace Chaos::LowLevelTest
{
	using FConvexGeometry = UE::Physics::FConvexGeometry;
	using FConvexGeometrySetup = UE::Physics::FConvexGeometrySetup;

	TEST_CASE("ConvexGeometry::Basic", "[Chaos][API][Geometry][unit]")
	{
		const FConvexGeometrySetup Setup = UE::Physics::MakeConvexBoxGeometrySetup();
		FConvexGeometrySetup MutableSetup = Setup;
		FConvexGeometry Convex(MoveTemp(MutableSetup));
		CHECK(Convex.GetNumVertices() == Setup.Vertices.Num());
		CHECK(Convex.GetMargin() == Setup.Margin);
	}
} // namespace Chaos::LowLevelTest

#endif // UE_RIGIDPHYSICS_API_ENABLED
