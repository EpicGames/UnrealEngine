// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/LowLevelTest/ChaosTestHarness.h"
#include "Chaos/LowLevelTest/ChaosTestScene.h"
#include "ChaosRigidPhysicsAsyncTest.h"
#include "ChaosRigidPhysicsAsync/RigidFactoryAsync.h"
#include "ChaosRigidPhysicsAsync/RigidGeometryCollectionAsync.h"
#include "ChaosRigidPhysicsAsync/RigidSceneAsync.h"
#include "ChaosRigidPhysicsAsync/RigidSceneSettingsAsync.h"
#include "RigidPhysics/RigidBodyContainer.h"
#include "RigidPhysics/RigidContext.h"

#if UE_RIGIDPHYSICS_API_ENABLED

namespace Chaos::LowLevelTest
{
	using namespace Chaos;
	using namespace Chaos::Rigids::Async;
	using namespace UE::Physics;

	FSimulationParameters MakeGCCubeSettings()
	{
		// TODO_CHAOSAPI: A lot of code required here
		return FSimulationParameters();
	}



	// Test casting between GC and BodyPool (both types of body container).
	// Casts from BodyContainer to/from GC should compile, but fail gracefully when the
	// types don't match.
	TEST_CASE("GeometryCollectionCastingTests", "[Chaos][API][GeometryCollection][unit]")
	{
		FRigidFactoryAsync Factory;
		FRigidSceneSettingsAsync SceneSettings;
		FRigidSceneHandle SceneHandle = Factory.CreateScene(FRigidDebugName::Make(TEXT("Main")), &SceneSettings);

		// We can safely (fail to) cast an empty body container to a GC, giving a null GC ptr
		if (FRigidContextGameRW Context = SceneHandle.LockRW())
		{
			FRigidBodyContainerHandle BodyContainerHandle;
			CHECK_FALSE(BodyContainerHandle.Pin(Context).IsValid());
			TRigidGeometryCollectionPtr<FRigidContextGameRW> GCPtr = BodyContainerHandle.Pin(Context);
			CHECK_FALSE(GCPtr.IsValid());
		}

		// We can cast a valid GCPtr to a valid BodyContainerPtr
		if (FRigidContextGameRW Context = SceneHandle.LockRW())
		{
			// TODO_CHAOSAPI: GeometryCollection initialization and test objects
			FSimulationParameters GCSettings = MakeGCCubeSettings();
			if (TRigidGeometryCollectionPtr<FRigidContextGameRW> GCPtr = Context->CreateGeometryCollection(FRigidDebugName::Make(TEXT("GC1")), GCSettings))
			{
				CHECK(GCPtr->GetTypeId() == FRigidGeometryCollectionAsyncGT::GetStaticTypeId());

				// Create a GC Handle
				FRigidGeometryCollectionHandle GCHandle = GCPtr;
				CHECK(GCHandle.Pin(Context).IsValid());

				// The GC handle should also be a valid BodyContainer handle
				FRigidBodyContainerHandle BCHandle = GCHandle;
				TRigidBodyContainerPtr<FRigidContextGameRW> BCPtr2 = BCHandle.Pin(Context);
				CHECK(BCPtr2.IsValid());
				CHECK(BCPtr2->GetTypeId() == FRigidGeometryCollectionAsyncGT::GetStaticTypeId());

				// Recover the GCPtr
				TRigidGeometryCollectionPtr<FRigidContextGameRW> GCPtr2 = BCPtr2;
				CHECK(GCPtr2.IsValid());
				CHECK(GCPtr2->GetTypeId() == FRigidGeometryCollectionAsyncGT::GetStaticTypeId());

				Context->DestroyGeometryCollection(MoveTemp(GCPtr));
			}
		}


		Factory.DestroyScene(SceneHandle);
	}

	TEST_CASE("GeometryCollectionInitializationTest", "[Chaos][API][GeometryCollection][unit]")
	{
		FRigidFactoryAsync Factory;
		FRigidSceneSettingsAsync SceneSettings;
		FRigidSceneHandle SceneHandle = Factory.CreateScene(FRigidDebugName::Make(TEXT("Main")), &SceneSettings);

		if (FRigidContextGameRW Context = SceneHandle.LockRW())
		{
			FSimulationParameters GCSettings = MakeGCCubeSettings();
			TRigidGeometryCollectionPtr<FRigidContextGameRW> GCPtr = Context->CreateGeometryCollection(FRigidDebugName::Make(TEXT("GC1")), GCSettings);
			REQUIRE(GCPtr.IsValid());

			// We can recover the GC from its handle on the GT
			{
				FRigidGeometryCollectionHandle GCHandle = GCPtr;
				TRigidGeometryCollectionPtr<FRigidContextGameRW> GCPtr2 = Context->GetGeometryCollection(GCHandle.GetId());
				CHECK(GCPtr2.IsValid());
				CHECK(GCPtr2.Get() == GCPtr.Get());
			}

			// We can access the GC as a BodyContainer on the GT
			{
				FRigidGeometryCollectionHandle GCHandle = GCPtr;
				TRigidBodyContainerPtr<FRigidContextGameRW> BodyContainerPtr = Context->GetBodyContainer(GCHandle.GetId());
				CHECK(BodyContainerPtr.IsValid());
				CHECK(BodyContainerPtr.Get() == GCPtr.Get());
			}

			// TODO: Fix eventually...
			// The GC has at least one body
			//CHECK(GCPtr->GetNumBodies() > 0);
			//TRigidBodyPtr<FRigidContextGameRW> BodyPtr = GCPtr->GetBodyAt(0);
			//CHECK(BodyPtr.IsValid());

			// A body from the GC should be a GC Body
			//if (BodyPtr.IsValid())
			//{
			//	CHECK(BodyPtr->GetTypeId() == FRigidGeometryCollectionBodyAsyncGT::GetStaticTypeId());
			//}

			Context->DestroyGeometryCollection(MoveTemp(GCPtr));
		}

		Factory.DestroyScene(SceneHandle);
	}
}

#endif