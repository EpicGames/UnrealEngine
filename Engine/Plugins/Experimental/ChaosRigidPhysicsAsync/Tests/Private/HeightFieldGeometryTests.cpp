// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/LowLevelTest/ChaosTestHarness.h"
#include "ChaosRigidPhysicsAsyncTest.h"

#include "RigidPhysics/Geometry/HeightFieldGeometry.h"
#include "RigidTestUtils.h"

#if UE_RIGIDPHYSICS_API_ENABLED

namespace Chaos::LowLevelTest
{
	using FHeightFieldGeometry = UE::Physics::FHeightFieldGeometry;
	using FHeightFieldGeometrySetup = UE::Physics::FHeightFieldGeometrySetup;

	FHeightFieldGeometrySetup MakeHeightFieldGeometrySetup()
	{
		FHeightFieldGeometrySetup Setup
		{
			.Heights
			{
				0, 1, 2,
				3, 4, 5,
				6, 7, 8
			},
			.MaterialIndices = { 0, 1, 2, 3 },
			.NumRows = 3,
			.NumCols = 3,
			.Scale = FVector3f(100.f, 100.f, 100.f),
		};
		return Setup;
	}

	TEST_CASE("HeightFieldGeometry::Basic", "[Chaos][API][Geometry][unit]")
	{
		const FHeightFieldGeometrySetup Setup = MakeHeightFieldGeometrySetup();
		FHeightFieldGeometrySetup MutableSetup = Setup;

		FHeightFieldGeometry Geometry(MoveTemp(MutableSetup));

		const int32 NumQuads = (Setup.NumRows - 1) * (Setup.NumCols - 1);
		REQUIRE(Geometry.GetNumRows() == Setup.NumRows);
		REQUIRE(Geometry.GetNumCols() == Setup.NumCols);
		CHECK_THAT(Geometry.GetScale(), Catch::ApproxEq(Setup.Scale));
		for (int32 I = 0; I < Setup.Heights.Num(); ++I)
		{
			CHECK_THAT(Setup.Heights[I], Catch::Matchers::WithinRel(Geometry.GetHeight(I), 0.001f));
		}
		for (int32 I = 0; I < Setup.MaterialIndices.Num(); ++I)
		{
			CHECK(Setup.MaterialIndices[I] == Geometry.GetMaterialIndex(I));
		}
	}
} // namespace Chaos::LowLevelTest

#endif // UE_RIGIDPHYSICS_API_ENABLED
