// Copyright Epic Games, Inc. All Rights Reserved.

#include "catch2/generators/catch_generators.hpp"
#include "Chaos/LowLevelTest/ChaosTestHarness.h"
#include "Chaos/LowLevelTest/ChaosTestScene.h"

#if UE_RIGIDPHYSICS_API_ENABLED

#include "ChaosRigidPhysicsAsyncTest.h"
#include "RigidPhysics/RigidBody.h"
#include "RigidPhysics/RigidContext.h"
#include "RigidPhysics/RigidScene.h"
#include "RigidPhysics/RigidShapeInstance.h"
#include "RigidPhysics/JointConstraint6DOF.h"
#include "RigidTestFixture.h"
#include "RigidTestUtils.h"

namespace Chaos::LowLevelTest
{
	using namespace Chaos;
	using namespace Chaos::Rigids::Async;
	using namespace UE::Physics;
	using EJointMotionType = UE::Physics::EJointMotionType;
	
	struct FJointFixture : public FRigidTestFixture
	{
		FRigidBodyHandle CreateSimpleBox(FRigidContextGameRW& Context, ERigidMovementType MovementType, const FVector& Location, bool bAutoCleanup = true, const FString& DebugName = "Body")
		{
			TRigidBodyPtr<FRigidContextGameRW> BodyPtr = Context->CreateBody(FRigidDebugName(DebugName), MovementType);
			REQUIRE(BodyPtr);
			BodyPtr->CreateShape(MakeBoxShape(FVector3f(100, 100, 100)));
			BodyPtr->InitTransform(FTransform(Location));
			if (MovementType == ERigidMovementType::Dynamic)
			{
				BodyPtr->SetIsSleeping(false);
				BodyPtr->SetMass(1000);
				BodyPtr->SetInertia(FVector3d(MakeSolidBoxInertia(1000, FVector3f(100, 100, 100))));
			}
			BodyPtr->Activate();
			if (bAutoCleanup)
			{
				AutoCleanup(BodyPtr);
			}
			return BodyPtr;
		}
	};

	TEST_CASE("JointConstraint6DOF: Construct Destruct", "[Chaos][API][JointConstraint][unit]")
	{
		FJointFixture Fixture;

		FJointConstraint6DOFHandle JointHandle;
		if (FRigidContextGameRW Context = Fixture.SceneHandle.LockRW())
		{
			TJointConstraint6DOFPtr<FRigidContextGameRW> Constraint = Context->CreateJointConstraint6DOF();
			REQUIRE(Constraint);
			JointHandle = Constraint;
		}

		if (FRigidContextGameRO Context = Fixture.SceneHandle.LockRO())
		{
			TJointConstraint6DOFPtr<FRigidContextGameRO> Constraint = JointHandle.Pin(Context);
			CHECK(Constraint);
		}

		if (FRigidContextGameRW Context = Fixture.SceneHandle.LockRW())
		{
			TJointConstraint6DOFPtr<FRigidContextGameRW> Constraint = JointHandle.Pin(Context);
			CHECK(Constraint);
			Context->DestroyJointConstraint(MoveTemp(Constraint));

			CHECK(!JointHandle.Pin(Context));
		}
	}
	
	TEST_CASE("JointConstraint6DOF: PT Pin", "[Chaos][API][JointConstraint][unit]")
	{
		FJointFixture Fixture;

		FJointConstraint6DOFHandle JointHandle;
		if (FRigidContextGameRW Context = Fixture.SceneHandle.LockRW())
		{
			TJointConstraint6DOFPtr<FRigidContextGameRW> Constraint = Context->CreateJointConstraint6DOF();
			Constraint->SetLinearLimit(123);
			JointHandle = Constraint;
			Fixture.AutoCleanup(JointHandle);
		}
		
		Fixture.RunPTCallback([&JointHandle](const FRigidContextSimRW& Context)
			{
				TJointConstraint6DOFPtr<FRigidContextSimRW> Constraint = JointHandle.Pin(Context);
				REQUIRE(Constraint);

				CHECK(Constraint->GetLinearLimit() == 123);
			});
	}
	
	TEST_CASE("JointConstraint6DOF: Misc Properties", "[Chaos][API][JointConstraint][unit]")
	{
		FJointFixture Fixture;

		FJointConstraint6DOFHandle JointHandle;
		if (FRigidContextGameRW Context = Fixture.SceneHandle.LockRW())
		{
			JointHandle = Context->CreateJointConstraint6DOF();
			Fixture.AutoCleanup(JointHandle);
		}

		if (FRigidContextGameRW Context = Fixture.SceneHandle.LockRW())
		{
			TJointConstraint6DOFPtr<FRigidContextGameRW> Constraint = JointHandle.Pin(Context);
			SECTION("LinearLimit")
			{
				Constraint->SetLinearLimit(150);
				CHECK(Constraint->GetLinearLimit() == 150);
			}
			SECTION("LinearMotionTypes")
			{
				const EJointMotionType MotionType = GENERATE(EJointMotionType::Free, EJointMotionType::Limited, EJointMotionType::Locked);

				Constraint->SetLinearMotionTypesX(MotionType);
				CHECK(Constraint->GetLinearMotionTypesX() == MotionType);
				Constraint->SetLinearMotionTypesY(MotionType);
				CHECK(Constraint->GetLinearMotionTypesY() == MotionType);
				Constraint->SetLinearMotionTypesZ(MotionType);
				CHECK(Constraint->GetLinearMotionTypesZ() == MotionType);
			}
		}
	}

	TEST_CASE("JointConstraint6DOF: Up/Down Casting", "[Chaos][API][JointConstraint][unit]")
	{
		FJointFixture Fixture;

		FJointConstraint6DOFHandle JointHandle6DOF;
		if (FRigidContextGameRW Context = Fixture.SceneHandle.LockRW())
		{
			JointHandle6DOF = Context->CreateJointConstraint6DOF();
			Fixture.AutoCleanup(JointHandle6DOF);
		}		

		if (FRigidContextGameRO Context = Fixture.SceneHandle.LockRO())
		{
			// Handle Up-cast
			FJointConstraintHandle JointHandle = JointHandle6DOF;
			REQUIRE(JointHandle.Pin(Context));
			// No Handle downcast as that requires pinning

			// Ptr casts:
			// valid ptr downcast
			CHECK(TJointConstraint6DOFPtr<FRigidContextGameRO>(JointHandle.Pin(Context)));
			// ptr upcast not yet supported
			// CHECK(TJointConstraintPtr<FRigidContextGameRO>(JointHandle6DOF.Pin(Context)));
			// No way to do invalid downcast currently
		}
	}
	
	TEST_CASE("JointConstraint6DOF: Simulate Basic", "[Chaos][API][JointConstraint][unit]")
	{
		FJointFixture Fixture;

		FJointConstraint6DOFHandle JointHandle;
		FRigidBodyHandle Body0Handle;
		FRigidBodyHandle Body1Handle;
		if (FRigidContextGameRW Context = Fixture.SceneHandle.LockRW())
		{
			Body0Handle = Fixture.CreateSimpleBox(Context, ERigidMovementType::Static, FVector::Zero());
			Body1Handle = Fixture.CreateSimpleBox(Context, ERigidMovementType::Dynamic, FVector(0, 0, -150));
			TJointConstraint6DOFPtr<FRigidContextGameRW> Constraint = Context->CreateJointConstraint6DOF();
			REQUIRE(Constraint);
			JointHandle = Constraint;
			Fixture.AutoCleanup(JointHandle);

			Constraint->SetBodies(Body0Handle.Pin(Context), Body1Handle.Pin(Context));
			FTransform Local1, Local2;
			Constraint->SetJointTransforms(Local1, Local2);
			Constraint->SetLinearLimit(150);
			Constraint->SetLinearMotionTypesX(EJointMotionType::Limited);
			Constraint->SetLinearMotionTypesY(EJointMotionType::Limited);
			Constraint->SetLinearMotionTypesZ(EJointMotionType::Limited);
			Constraint->Activate();
		}

		if (FRigidContextGameRW Context = Fixture.SceneHandle.LockRW())
		{
			TRigidBodyPtr<FRigidContextGameRW> Body1Ptr = Body1Handle.Pin(Context);
			for (int32 TickIndex = 0; TickIndex < 10; ++TickIndex)
			{
				Context->StartTick(0.01f);
				Context->EndTick();
			}

			CHECK(Body1Ptr->GetTransform().GetLocation().Z > -151);
		}
	}
} // namespace Chaos::LowLevelTest

#endif // UE_RIGIDPHYSICS_API_ENABLED
