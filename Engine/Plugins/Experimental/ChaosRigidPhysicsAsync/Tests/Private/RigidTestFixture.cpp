// Copyright Epic Games, Inc. All Rights Reserved.
#include "RigidTestFixture.h"

#include "Chaos/LowLevelTest/ChaosTestHarness.h"
#include "Chaos/LowLevelTest/ChaosTestScene.h"

#if UE_RIGIDPHYSICS_API_ENABLED

#include "ChaosRigidPhysicsAsync/RigidFactoryAsync.h"
#include "ChaosRigidPhysicsAsync/RigidBodyAsync.h"
#include "ChaosRigidPhysicsAsync/RigidGeometryCollectionAsync.h"
#include "ChaosRigidPhysicsAsync/RigidSceneAsync.h"

#include "RigidPhysics/RigidBody.h"
#include "RigidPhysics/RigidContext.h"
#include "RigidPhysics/RigidShapeInstance.h"
#include "TestSceneModifier.h"
#include "RigidTestUtils.h"

namespace Chaos::LowLevelTest
{
	using namespace Chaos;
	using namespace Chaos::Rigids::Async;
	using namespace UE::Physics;

	FRigidTestFixture::FRigidTestFixture()
	{
		SceneHandle = Factory.CreateScene(FRigidDebugName::Make(TEXT("Main")), &SceneSettings);
	}

	FRigidTestFixture::~FRigidTestFixture()
	{
		if (FRigidContextGameRW Context = SceneHandle.LockRW())
		{
			for (FJointConstraintHandle& JointHandle : JointHandles)
			{
				Context->DestroyJointConstraint(JointHandle.Pin(Context));
			}
			for (FRigidBodyHandle& BodyHandle : BodyHandles)
			{
				Context->DestroyBody(BodyHandle.Pin(Context));
			}
			for (FRigidGeometryCollectionHandle& GCHandle : GeometryCollectionHandles)
			{
				Context->DestroyGeometryCollection(GCHandle.Pin(Context));
			}
		}
		for (FMaterialHandle& MaterialHandle : MaterialHandles)
		{
			FPhysicalMaterialManager::Get().Destroy(MaterialHandle.GetMaterialHandle());
		}
		for (FMaterialMaskHandle& MaterialMaskHandle : MaterialMaskHandles)
		{
			FPhysicalMaterialManager::Get().Destroy(MaterialMaskHandle.GetMaterialMaskHandle());
		}
		Factory.DestroyScene(SceneHandle);
	}

	FRigidContextGameRO FRigidTestFixture::LockROChecked() const
	{
		// Note: This isn't 100% identical to checking the context, but it's effectively the same and there's no other way to write this.
		CHECK(SceneHandle.GetScene());
		return SceneHandle.LockRO();
	}

	FRigidContextGameRW FRigidTestFixture::LockRWChecked() const
	{
		// Note: This isn't 100% identical to checking the context, but it's effectively the same and there's no other way to write this.
		CHECK(SceneHandle.GetScene());
		return SceneHandle.LockRW();
	}

	FMaterialHandle FRigidTestFixture::CreateMaterial(bool bAutoCleanup)
	{
		FMaterialHandle MaterialHandle(FPhysicalMaterialManager::Get().Create());
		if (bAutoCleanup)
		{
			MaterialHandles.Add(MaterialHandle);
		}
		return MaterialHandle;
	}

	FMaterialMaskHandle FRigidTestFixture::CreateMaterialMask(bool bAutoCleanup)
	{
		FMaterialMaskHandle MaterialMaskHandle(FPhysicalMaterialManager::Get().CreateMask());
		if (bAutoCleanup)
		{
			MaterialMaskHandles.Add(MaterialMaskHandle);
		}
		return MaterialMaskHandle;
	}

	void FRigidTestFixture::AutoCleanup(FJointConstraintHandle Handle)
	{
		JointHandles.Add(Handle);
	}

	void FRigidTestFixture::AutoCleanup(FRigidBodyHandle Handle)
	{
		BodyHandles.Add(Handle);
	}

	void FRigidTestFixture::AutoCleanup(FRigidGeometryCollectionHandle Handle)
	{
		GeometryCollectionHandles.Add(Handle);
	}

	void FRigidTestFixture::RunPTCallback(TFunction<void(const FRigidContextSimRW&)> Callback, float Dt, int32 Iterations)
	{
		FTestRigidSceneModifier SceneModifier;
		SceneModifier.PreTickFunc = Callback;

		if (FRigidContextGameRW Context = LockRWChecked())
		{
			Context->RegisterModifier(&SceneModifier);
			for (int32 I = 0; I < Iterations; ++I)
			{
				Context->StartTick(Dt);
				Context->EndTick();
			}
			Context->UnregisterModifier(&SceneModifier);
		}
	}
} // namespace Chaos::LowLevelTest

#endif // UE_RIGIDPHYSICS_API_ENABLED
