// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "RigidPhysics/RigidFwd.h"
#include "ChaosRigidPhysicsAsync/RigidFactoryAsync.h"
#include "ChaosRigidPhysicsAsync/RigidSceneSettingsAsync.h"
#include "RigidPhysics/RigidScene.h"
#include "RigidPhysics/RigidMaterials.h"

#if UE_RIGIDPHYSICS_API_ENABLED

namespace Chaos::LowLevelTest
{
	using FRigidDebugName = UE::Physics::FRigidDebugName;
	using FRigidContextGameRW = UE::Physics::FRigidContextGameRW;
	using FRigidContextSimRW = UE::Physics::FRigidContextSimRW;
	using ERigidMovementType = UE::Physics::ERigidMovementType;
	using FJointConstraintHandle = UE::Physics::FJointConstraintHandle;
	using FRigidBodyHandle = UE::Physics::FRigidBodyHandle;
	using FRigidGeometryCollectionHandle = UE::Physics::FRigidGeometryCollectionHandle;
	using FRigidContextGameRO = UE::Physics::FRigidContextGameRO;
	using FRigidContextGameRW = UE::Physics::FRigidContextGameRW;
	using FMaterialHandle = UE::Physics::FMaterialHandle;
	using FMaterialMaskHandle = UE::Physics::FMaterialMaskHandle;

	struct FRigidTestFixture
	{
		FRigidTestFixture();
		~FRigidTestFixture();

		FRigidContextGameRO LockROChecked() const;
		FRigidContextGameRW LockRWChecked() const;

		FMaterialHandle CreateMaterial(bool bAutoCleanup = true);
		FMaterialMaskHandle CreateMaterialMask(bool bAutoCleanup = true);

		void AutoCleanup(FJointConstraintHandle Handle);
		void AutoCleanup(FRigidBodyHandle Handle);
		void AutoCleanup(FRigidGeometryCollectionHandle Handle);

		void RunPTCallback(TFunction<void(const FRigidContextSimRW&)> Callback, float Dt = 0.01, int32 Iterations = 1);

		Chaos::Rigids::Async::FRigidFactoryAsync Factory;
		Chaos::Rigids::Async::FRigidSceneSettingsAsync SceneSettings;
		UE::Physics::FRigidSceneHandle SceneHandle;

		TArray<FRigidBodyHandle> BodyHandles;
		TArray<FRigidGeometryCollectionHandle> GeometryCollectionHandles;
		TArray<FJointConstraintHandle> JointHandles;
		TArray<FMaterialHandle> MaterialHandles;
		TArray<FMaterialMaskHandle> MaterialMaskHandles;
	};
} // namespace Chaos::LowLevelTest

#endif // UE_RIGIDPHYSICS_API_ENABLED
