// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigidTestUtils.h"

#if UE_RIGIDPHYSICS_API_ENABLED

#include "PhysicalMaterials/PhysicalMaterial.h"

namespace UE::Physics
{
	FTriangleMeshGeometrySetup MakeSingleTriangleGeometrySetup(const FVector3f& InScale, const FVector3f& InTranslation)
	{
		FTriangleMeshGeometrySetup Setup
		{
			.Vertices
			{
				FVector3f(0, 0, 0) * InScale + InTranslation,
				FVector3f(1, 0, 0) * InScale + InTranslation,
				FVector3f(0, 1, 0) * InScale + InTranslation,
			},
			.TriangleIndices
			{
				FInt32Vector3(0, 1, 2),
			},
			.MaterialIndices{0},
		};
		return Setup;
	}

	FTriangleMeshGeometrySetup MakeQuadTriangleGeometrySetup(const FVector3f& InScale, const FVector3f& InTranslation)
	{
		FTriangleMeshGeometrySetup Setup
		{
			.Vertices
			{
				FVector3f(0, 0, 0) * InScale + InTranslation,
				FVector3f(1, 0, 0) * InScale + InTranslation,
				FVector3f(1, 1, 0) * InScale + InTranslation,
				FVector3f(0, 1, 0) * InScale + InTranslation,
			},
			.TriangleIndices
			{
				FInt32Vector3(0, 1, 2),
				FInt32Vector3(0, 2, 3),
			},
			.MaterialIndices{0, 1}
		};
		return Setup;
	}
	
	FConvexGeometrySetup MakeConvexBoxGeometrySetup(const FVector3f& InCenter, const FVector3f& InHalfExtent, const float InMargin)
	{
		FConvexGeometrySetup Setup
		{
			.Vertices
			{
				InCenter + InHalfExtent * FVector3f(-1, -1, -1),
				InCenter + InHalfExtent * FVector3f( 1, -1, -1),
				InCenter + InHalfExtent * FVector3f(-1,  1, -1),
				InCenter + InHalfExtent * FVector3f( 1,  1, -1),
				InCenter + InHalfExtent * FVector3f(-1, -1,  1),
				InCenter + InHalfExtent * FVector3f( 1, -1,  1),
				InCenter + InHalfExtent * FVector3f(-1,  1,  1),
				InCenter + InHalfExtent * FVector3f( 1,  1,  1),
			},
			.Margin = InMargin
		};
		return Setup;
	}

	FHeightFieldGeometrySetup MakeHeightFieldGeometrySetup()
	{
		return FHeightFieldGeometrySetup
		{
			.Heights
			{
				1, 0, 1,
				0, 2, 0,
				1, 0, 1
			},
			.MaterialIndices = { 0, 1, 2, 3 },
			.NumRows = 3,
			.NumCols = 3,
			.Scale = FVector3f::OneVector,
		};
	}

	FRigidShapeInstanceSetup MakeBoxShape(const FVector3f& InSize, const FVector3f& InCenter, UPhysicalMaterial* InMaterial)
	{
		FBoxGeometry BoxGeom(InSize);
		FRigidShapeInstanceSetup Setup(BoxGeom);
		Setup.LocalTransform = FTransform3f(InCenter);
		// TODO: Material
		return Setup;
	}

	FVector3f MakeSolidBoxInertia(float InMass, const FVector3f& InSize)
	{
		FVector3f SizeSq = InSize * InSize;
		return (InMass / 12.0f) * FVector3f(
			SizeSq.Y + SizeSq.Z,
			SizeSq.X + SizeSq.Z,
			SizeSq.X + SizeSq.Y);
	}
} // namespace UE::Physics

#endif // UE_RIGIDPHYSICS_API_ENABLED
