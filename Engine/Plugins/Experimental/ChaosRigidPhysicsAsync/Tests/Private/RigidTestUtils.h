// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_RIGIDPHYSICS_API_ENABLED

#include "Chaos/ImplicitFwd.h"
#include "Math/Bounds.h"
#include "Math/Transform.h"
#include "RigidPhysics/RigidLog.h"
#include "RigidPhysics/RigidShapeInstanceSetup.h"

class UPhysicalMaterial;

namespace Chaos
{
	class FPerShapeData;
} // namespace Chaos

namespace UE::Physics
{
	FTriangleMeshGeometrySetup MakeSingleTriangleGeometrySetup(const FVector3f& InScale = FVector3f::One(), const FVector3f& InTranslation = FVector3f::Zero());
	FTriangleMeshGeometrySetup MakeQuadTriangleGeometrySetup(const FVector3f& InScale = FVector3f::One(), const FVector3f& InTranslation = FVector3f::Zero());
	FConvexGeometrySetup MakeConvexBoxGeometrySetup(const FVector3f& InCenter = FVector3f::Zero(), const FVector3f& InHalfExtent = FVector3f::One(), const float InMargin = 0);
	FHeightFieldGeometrySetup MakeHeightFieldGeometrySetup();
	FRigidShapeInstanceSetup MakeBoxShape(const FVector3f& InSize, const FVector3f& InCenter = FVector3f::Zero(), UPhysicalMaterial* InMaterial = nullptr);
	FVector3f MakeSolidBoxInertia(float InMass, const FVector3f& InSize);
} // namespace UE::Physics

#endif // UE_RIGIDPHYSICS_API_ENABLED
