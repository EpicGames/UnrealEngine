// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/LowLevelTest/ChaosTestHarness.h"
#include "Chaos/LowLevelTest/ChaosTestScene.h"
#include "ChaosRigidPhysicsAsync/RigidFactoryAsync.h"
#include "ChaosRigidPhysicsAsync/RigidSceneSettingsAsync.h"
#include "ChaosRigidPhysicsAsync/RigidFwdAsync.h"
#include "PBDRigidsSolver.h"
#include "PhysicsInterfaceDeclaresCore.h"
#include "Physics/Experimental/PhysInterface_Chaos.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"
#include "RigidPhysics/RigidBody.h"
#include "RigidPhysics/RigidContext.h"
#include "RigidPhysics/RigidShapeInstance.h"

#include "RigidTestUtils.h"

#if UE_RIGIDPHYSICS_API_ENABLED

namespace Chaos::LowLevelTest
{

	// This test is only intended to be run with an attached sampling profiler.
	// It loops for a very long time to faciliate sampling, so don't enable it for regular testing.
	TEST_CASE("ChaosAPIBenchmark", "[.LoopForever][ChaosRBPerBodyLock]")
	{
		using namespace Chaos;
		using namespace Chaos::Rigids::Async;
		using namespace UE::Physics;

		int32 NumBodies = 10000;
		FTransform Transform = FTransform::Identity;

		FRigidFactoryAsync Factory;
		FRigidSceneSettingsAsync SceneSettings;
		FRigidSceneHandle SceneHandle = Factory.CreateScene(FRigidDebugName::Make(TEXT("Main")), &SceneSettings);

		// Create the bodies
		TArray<FRigidBodyHandle> BodyHandles;

		if (FRigidContextGameRW Context = SceneHandle.LockRW())
		{
			BodyHandles.Reserve(NumBodies);
			for (int32 BodyIndex = 0; BodyIndex < NumBodies; ++BodyIndex)
			{
				TRigidBodyPtr<FRigidContextGameRW> BodyPtr = Context->CreateBody(FRigidDebugName::Make(TEXT("Body{0}"), BodyIndex), ERigidMovementType::Dynamic);
				BodyPtr->CreateShape(MakeBoxShape(FVector3f(50, 50, 50)));
				BodyPtr->Activate();

				BodyHandles.Add(BodyPtr);
			}
		}

		// Change transform to prevent early-out
		for (int32 LoopIndex = 0; LoopIndex < 1000000; ++LoopIndex)
		{
			FVector Pos = Transform.GetTranslation() + FVector(100, 0, 0);
			Transform.SetTranslation(Pos);

			if (FRigidContextGameRW Context = SceneHandle.LockRW())
			{
				for (FRigidBodyHandle BodyHandle : BodyHandles)
				{
					if (TRigidBodyPtr<FRigidContextGameRW> Body = BodyHandle.Pin(Context))
					{
						Body->InitTransform(Transform);
					}
				}
			}
		}

		if (FRigidContextGameRW Context = SceneHandle.LockRW())
		{
			for (FRigidBodyHandle BodyHandle : BodyHandles)
			{
				Context->DestroyBody(BodyHandle.Pin(Context));
			}
		}
		BodyHandles.Reset();

		Factory.DestroyScene(SceneHandle);
	}

	// This test is only intended to be run with an attached sampling profiler.
	// It loops for a very long time to faciliate sampling, so don't enable it for regular testing.
	TEST_CASE("ChaosAPIBenchmark ProxyComparison", "[.LoopForever][ChaosPerBodyLock]")
	{
		using namespace Chaos;
		using namespace UE::Physics;

		int32 NumBodies = 10000;
		FTransform Transform = FTransform::Identity;

		FChaosTestScene Scene;

		// Create the bodies
		TArray<FSingleParticlePhysicsProxy*> Proxies;
		Proxies.Reserve(NumBodies);
		for (int32 BodyIndex = 0; BodyIndex < NumBodies; ++BodyIndex)
		{
			FSingleParticlePhysicsProxy* Proxy = Scene.AddDynamicBox(TEXT("Box"), Transform, FVector(50, 50, 50));
			Proxies.Add(Proxy);
		}

		for (int32 LoopIndex = 0; LoopIndex < 1000000; ++LoopIndex)
		{
			FVector Pos = Transform.GetTranslation() + FVector(100, 0, 0);
			Transform.SetTranslation(Pos);

			for (FSingleParticlePhysicsProxy* Proxy : Proxies)
			{
				FPhysicsCommand::ExecuteWrite(Proxy, 
					[Proxy, &Transform](const FPhysicsActorHandle& Actor)
					{
						FPhysicsInterface::SetGlobalPose_AssumesLocked(Proxy, Transform);
					});
			}
		}

		Proxies.Reset();
	}



	TEST_CASE("ChaosAPIBenchmark BodyHandle Lock", "[!benchmark][ChaosAPIBenchmark]")
	{
		using namespace Chaos;
		using namespace Chaos::Rigids::Async;
		using namespace UE::Physics;

#if UE_RIGIDPHYSICS_CHECK_ENABLED
		UE_LOGF(LogRigidPhysics, Error, "Benchmarks with asserts enabled will not give best results");
#endif

		int32 NumBodies = 3000;
		FTransform Transform = FTransform::Identity;

		FRigidFactoryAsync Factory;
		FRigidSceneSettingsAsync SceneSettings;
		FRigidSceneHandle SceneHandle = Factory.CreateScene(FRigidDebugName::Make(TEXT("Main")), &SceneSettings);

		// Create the bodies
		TArray<FRigidBodyHandle> BodyHandles;

		if (FRigidContextGameRW Context = SceneHandle.LockRW())
		{
			BodyHandles.Reserve(NumBodies);
			for (int32 BodyIndex = 0; BodyIndex < NumBodies; ++BodyIndex)
			{
				TRigidBodyPtr<FRigidContextGameRW> BodyPtr = Context->CreateBody(FRigidDebugName::Make(TEXT("Body{0}"), BodyIndex), ERigidMovementType::Dynamic);
				BodyPtr->CreateShape(MakeBoxShape(FVector3f(50, 50, 50)));
				BodyPtr->Activate();

				BodyHandles.Add(BodyPtr);
			}
		}

		BENCHMARK("Per-Body Lock")
		{
			// Change transform to prevent early-out
			Transform.SetTranslation(Transform.GetTranslation() + FVector(100, 0, 0));

			for (FRigidBodyHandle BodyHandle : BodyHandles)
			{
				if (FRigidContextGameRW Context = SceneHandle.LockRW())
				{
					if (TRigidBodyPtr<FRigidContextGameRW> Body = BodyHandle.Pin(Context))
					{
						Body->InitTransform(Transform);
					}
				}
			}
		};

		BENCHMARK("Scene Lock")
		{
			// Change transform to prevent early-out
			Transform.SetTranslation(Transform.GetTranslation() + FVector(100, 0, 0));

			if (FRigidContextGameRW Context = SceneHandle.LockRW())
			{
				for (FRigidBodyHandle BodyHandle : BodyHandles)
				{
					// Pin does not add a lock - we are already in a scene lock
					if (TRigidBodyPtr<FRigidContextGameRW> Body = BodyHandle.Pin(Context))
					{
						Body->InitTransform(Transform);
					}
				}
			}
		};

		if (FRigidContextGameRW Context = SceneHandle.LockRW())
		{
			for (FRigidBodyHandle BodyHandle : BodyHandles)
			{
				Context->DestroyBody(BodyHandle.Pin(Context));
			}
		}
		BodyHandles.Reset();

		Factory.DestroyScene(SceneHandle);
	}

	TEST_CASE("ChaosAPIBenchmark BodyHandle Lock ProxyComparison", "[!benchmark][ChaosAPIBenchmark]")
	{
		using namespace Chaos;
		using namespace UE::Physics;

		int32 NumBodies = 3000;
		FTransform Transform = FTransform::Identity;

		FChaosTestScene Scene;

		// Create the bodies
		TArray<FSingleParticlePhysicsProxy*> Proxies;
		Proxies.Reserve(NumBodies);
		for (int32 BodyIndex = 0; BodyIndex < NumBodies; ++BodyIndex)
		{
			FSingleParticlePhysicsProxy* Proxy = Scene.AddDynamicBox(TEXT("Box"), Transform, FVector(50,50,50));
			Proxies.Add(Proxy);
		}

		BENCHMARK("Per-Body Lock")
		{
			// Change transform to prevent early-out
			Transform.SetTranslation(Transform.GetTranslation() + FVector(100, 0, 0));

			for (FSingleParticlePhysicsProxy* Proxy : Proxies)
			{
				FPhysicsCommand::ExecuteWrite(Proxy,
					[Proxy, &Transform](const FPhysicsActorHandle& Actor)
					{
						FPhysicsInterface::SetGlobalPose_AssumesLocked(Proxy, Transform);
					});
			}
		};

		BENCHMARK("Scene Lock")
		{
			// Change transform to prevent early-out
			Transform.SetTranslation(Transform.GetTranslation() + FVector(100, 0, 0));

			FPhysicsCommand::ExecuteWrite(Proxies[0],
				[Proxies, &Transform](const FPhysicsActorHandle& Actor)
				{
					for (FSingleParticlePhysicsProxy* Proxy : Proxies)
					{
						FPhysicsInterface::SetGlobalPose_AssumesLocked(Proxy, Transform);
					}
				});
		};

		Proxies.Reset();
	}

}

#endif