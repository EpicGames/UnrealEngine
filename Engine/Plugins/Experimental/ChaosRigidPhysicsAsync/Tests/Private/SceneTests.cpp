// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/Collision/ContactPoint.h"
#include "Chaos/LowLevelTest/ChaosTestErrorLogSuppressor.h"
#include "Chaos/LowLevelTest/ChaosTestHarness.h"
#include "Chaos/LowLevelTest/ChaosTestScene.h"
#include "ChaosRigidPhysicsAsyncTest.h"
#include "ChaosRigidPhysicsAsync/RigidFactoryAsync.h"
#include "ChaosRigidPhysicsAsync/RigidBodyAsync.h"
#include "ChaosRigidPhysicsAsync/RigidSceneSettingsAsync.h"
#include "Framework/Threading.h"
#include "HAL/Thread.h"
#include "RigidPhysics/RigidBody.h"
#include "RigidPhysics/RigidContext.h"
#include "RigidPhysics/RigidModifier.h"
#include "RigidPhysics/RigidScene.h"
#include "RigidPhysics/RigidShapeInstance.h"
#include "TestSceneModifier.h"

#include "RigidTestUtils.h"

#if UE_RIGIDPHYSICS_API_ENABLED

namespace Chaos::LowLevelTest
{
	using namespace Chaos;
	using namespace Chaos::Rigids::Async;
	using namespace UE::Physics;

	TEST_CASE("SceneLockTest", "[Chaos][API][Scene][unit]")
	{
		FRigidFactoryAsync SceneFactory;
		FRigidSceneSettingsAsync SceneSettings;
		FRigidSceneHandle SceneHandle = SceneFactory.CreateScene(FRigidDebugName::Make(TEXT("Main")), &SceneSettings);

		// Create a thread that will lock the physics scene.
		// We initially lock ThreadRunningLock on the main thread to prevent the thread from executing.
		// The main thread unlocks ThreadRunningLock after it has locked the scene, at which point the thread will try to lock the scene and will wait.
		// After verifying that the thread is waiting, the main thread unlocks the scene and allows the thread to run.
		// The thread then locks the scene and incrmement the ThreadLockCount.
		// The main thread will then wait until ThreadRunningLock is unlocked again before exiting, by which time the thread should been able to lock the scene.
		FCriticalSection ThreadRunningLock = FCriticalSection();
		std::atomic_bool bThreadRunning = false;
		std::atomic_int ThreadLockCount = 0;
		const auto& ThreadFunc =
			[SceneHandle, &ThreadRunningLock, &bThreadRunning, &ThreadLockCount]()
			{
#if PHYSICS_THREAD_CONTEXT
				FGameThreadContextScope GTScope(true);
#endif

				ThreadRunningLock.Lock();

				bThreadRunning = true;

				if (FRigidContextGameRW Context = SceneHandle.LockRW())
				{
					++ThreadLockCount;
				}

				FPlatformProcess::YieldThread();

				ThreadRunningLock.Unlock();
			};

		// Start the thread but make it wait until we lock the scene before it continues
		ThreadRunningLock.Lock();
		FThread Thread(TEXT("TestThread"), ThreadFunc);

		// Lock the scene on the main thread and then start the background thread.
		// The thread should be blocked on SceneHandle.LockRW() until we are done here. 
		if (FRigidContextGameRW Context = SceneHandle.LockRW())
		{
			// Allow the background thread to run
			ThreadRunningLock.Unlock();

			// Wait until the thread is running
			while (!bThreadRunning)
			{
				FPlatformProcess::YieldThread();
			}

			// Wait a bit longer - the thread should not be able to get the physics lock
			// TODO_CHAOSAPI: Can we check the thread/lock state to explicitly verify that the 
			// background thread is waiting on the lock?
			FPlatformProcess::Sleep(0.1f);

			// If this fails, then the above SceneHandle.LockRW() did not work
			CHECK(ThreadLockCount == 0);
		}

		// Now the thread should get the lock and increment the counter
		ThreadRunningLock.Lock();

		// If this fails, then the thread never gained the lock
		CHECK(ThreadLockCount > 0);

		ThreadRunningLock.Unlock();

		// Terminate the thread
		Thread.Join();

		SceneFactory.DestroyScene(SceneHandle);
	}


	TEST_CASE("CreateSceneAndBodies", "[Chaos][API][Scene][unit]")
	{
		float Dt = 1.0f / 60.0f;

		FRigidFactoryAsync SceneFactory;
		FRigidSceneSettingsAsync SceneSettings;
		FRigidSceneHandle SceneHandle = SceneFactory.CreateScene(FRigidDebugName::Make(TEXT("Main")), &SceneSettings);
		REQUIRE(SceneHandle.LockRO());
		if (FRigidContextGameRW Context = SceneHandle.LockRW())
		{
			REQUIRE(Context.GetScene().IsValid());
		}

		SECTION("EmptyBodyHandlesAreInvalid")
		{
			// Empty RigidBodyHandle
			FRigidBodyHandle BodyHandle;
			if (FRigidContextGameRW Context = SceneHandle.LockRW())
			{
				// Handle should be invalid (explicit and implicit checks)
				CHECK_FALSE(BodyHandle.Pin(Context).IsValid());
				CHECK_FALSE(BodyHandle.Pin(Context));
			}
		}

		SECTION("AddBodies")
		{
			FRigidBodyHandle StaticBodyHandle;
			FRigidBodyHandle DynamicBodyHandle;
			if (FRigidContextGameRW Context = SceneHandle.LockRW())
			{
				StaticBodyHandle = Context->CreateBody(FRigidDebugName::Make(TEXT("Body1")), ERigidMovementType::Static);
				DynamicBodyHandle = Context->CreateBody(FRigidDebugName::Make(TEXT("Body2")), ERigidMovementType::Dynamic);
				REQUIRE(StaticBodyHandle.Pin(Context));
				REQUIRE(DynamicBodyHandle.Pin(Context));

				// Check that we created objects of the correct type
				CHECK(StaticBodyHandle.Pin(Context)->GetTypeId() == FRigidBodyAsyncGT::GetStaticTypeId());
				CHECK(DynamicBodyHandle.Pin(Context)->GetTypeId() == FRigidBodyAsyncGT::GetStaticTypeId());

				// Add Collision
				StaticBodyHandle.Pin(Context)->CreateShape(MakeBoxShape(FVector3f(100, 100, 100)));
				DynamicBodyHandle.Pin(Context)->CreateShape(MakeBoxShape(FVector3f(100, 100, 100)));

				// Activate Bodies
				StaticBodyHandle.Pin(Context)->Activate();
				DynamicBodyHandle.Pin(Context)->Activate();
				CHECK(StaticBodyHandle.Pin(Context)->IsActive());
				CHECK(DynamicBodyHandle.Pin(Context)->IsActive());
			}

			SECTION("TickSystemWithBodies")
			{
				if (FRigidContextGameRW Context = SceneHandle.LockRW())
				{
					Context.GetScene()->StartTick(Dt);
					Context.GetScene()->EndTick();

					// Dynamic body should have moved under gravity
					if (TRigidBodyPtr<FRigidContextGameRW> BodyPtr = DynamicBodyHandle.Pin(Context))
					{
						FVector BodyLocation = BodyPtr->GetTransform().GetLocation();
						CHECK(BodyLocation.Z < 0.0f);
					}
				}
			}

			SECTION("ChangeBodyState")
			{
				if (FRigidContextGameRW Context = SceneHandle.LockRW())
				{
					if (TRigidBodyPtr<FRigidContextGameRW> BodyPtr = DynamicBodyHandle.Pin(Context))
					{
						BodyPtr->InitTransform(FTransform(FVector(100,0,0)));
						CHECK(BodyPtr->GetTransform().GetTranslation().X == 100.0f);
					}
				}
			}

			if (FRigidContextGameRW Context = SceneHandle.LockRW())
			{
				// Deactivate Bodies
				StaticBodyHandle.Pin(Context)->Deactivate();
				DynamicBodyHandle.Pin(Context)->Deactivate();
				CHECK_FALSE(StaticBodyHandle.Pin(Context)->IsActive());
				CHECK_FALSE(DynamicBodyHandle.Pin(Context)->IsActive());

				// Destroy Bodies
				Context->DestroyBody(StaticBodyHandle.Pin(Context));
				Context->DestroyBody(DynamicBodyHandle.Pin(Context));
				CHECK_FALSE(StaticBodyHandle.Pin(Context));
				CHECK_FALSE(DynamicBodyHandle.Pin(Context));
			}
		}

		SECTION("BodyHandleCopy")
		{
			if (FRigidContextGameRW Context = SceneHandle.LockRW())
			{
				// Create a body
				FRigidBodyHandle BodyHandle = Context->CreateBody(FRigidDebugName::Make(TEXT("Body1")), ERigidMovementType::Dynamic);
				CHECK(BodyHandle.Pin(Context));

				// Create a duplicate body handle - it should also be valid and reference the same body object
				FRigidBodyHandle BodyHandleDupe = BodyHandle;
				CHECK(BodyHandleDupe.Pin(Context));
				CHECK(BodyHandleDupe.Pin(Context).Get() == BodyHandle.Pin(Context).Get());

				// Add Collision
				BodyHandle.Pin(Context)->CreateShape(MakeBoxShape(FVector3f(100, 100, 100)));

				Context->StartTick(Dt);
				Context->EndTick();

				// Destroy the body
				Context->DestroyBody(BodyHandle.Pin(Context));

				// Body handle should is invalid (references destroyed body)
				CHECK_FALSE(BodyHandle.Pin(Context));
				CHECK_FALSE(BodyHandleDupe.Pin(Context));
			}
		}

		SECTION("DestroyBodyInvalidatesPointer")
		{
			if (FRigidContextGameRW Context = SceneHandle.LockRW())
			{
				// Create a body
				TRigidBodyPtr<FRigidContextGameRW> BodyPtr = Context->CreateBody(FRigidDebugName::Make(TEXT("Body1")), ERigidMovementType::Dynamic);
				REQUIRE(BodyPtr.IsValid());

				// The pointer should be invalidated by Destroy
				Context->DestroyBody(MoveTemp(BodyPtr));
				CHECK(!BodyPtr.IsValid());

				// We gracefull handle attempt to destroy again (should not crash or assert)
				Context->DestroyBody(MoveTemp(BodyPtr));
			}
		}

		SceneFactory.DestroyScene(SceneHandle);
	} // TEST_CASE

	TEST_CASE("CallbacksAreCalled", "[Chaos][API][Scene][unit]")
	{
		float Dt = 1.0f / 60.0f;

		std::atomic<int32> PreSimCallCount = 0;
		std::atomic<int32> PreTickCallCount = 0;
		FTestRigidSceneModifier SceneModifier;
		SceneModifier.PreSimFunc = 
			[&PreSimCallCount](const FRigidContextGameRW& Context)
			{
				++PreSimCallCount;
			};
		SceneModifier.PreTickFunc =
			[&PreTickCallCount](const FRigidContextSimRW& Context)
			{
				++PreTickCallCount;
			};

		FRigidFactoryAsync SceneFactory;
		FRigidSceneSettingsAsync SceneSettings;
		FRigidSceneHandle SceneHandle = SceneFactory.CreateScene(FRigidDebugName::Make(TEXT("Main")), &SceneSettings);

		if (FRigidContextGameRW Context = SceneHandle.LockRW())
		{
			Context->RegisterModifier(&SceneModifier);

			Context->StartTick(Dt);
			Context->EndTick();
			CHECK(PreSimCallCount == 1);
			CHECK(PreTickCallCount == 1);

			Context->StartTick(Dt);
			Context->EndTick();
			CHECK(PreSimCallCount == 2);
			CHECK(PreTickCallCount == 2);

			Context->UnregisterModifier(&SceneModifier);

			Context->StartTick(Dt);
			Context->EndTick();
			CHECK(PreSimCallCount == 2);
			CHECK(PreTickCallCount == 2);
		}

		SceneFactory.DestroyScene(SceneHandle);
	};

	// Change the position of Dynamic bodies from GT and PT callbacks 
	// and  verify that the changes are visible on the GT.
	// NOTE: InitTransform is currently not supported on the PT for 
	// Static and Kinematic bodies - the changes do not replicate
	// back to the GT.
	TEST_CASE("GameAndPhysicsThreadCallbacks", "[Chaos][API][Scene][unit]")
	{
		float Dt = 1.0f / 60.0f;

		FRigidBodyHandle BodyHandle1;
		FRigidBodyHandle BodyHandle2;
		FTestRigidSceneModifier SceneModifier;

		// GT Callback - Change the position of Body1
		SceneModifier.PreSimFunc =
			[&BodyHandle1](const FRigidContextGameRW& Context)
			{
				CHECK(BodyHandle1.Pin(Context));
				if (TRigidBodyPtr<FRigidContextGameRW> BodyPtr = BodyHandle1.Pin(Context))
				{
					// Make sure this is a GT body
					REQUIRE(BodyPtr->GetTypeId() == FRigidBodyAsyncGT::GetStaticTypeId());

					BodyPtr->InitTransform(FTransform(FVector(200.0, 0.0, 0.0)));
					CHECK(BodyPtr->GetTransform().GetLocation().X == 200.0);
				}
			};

		// PT Callback - Change the position of Body2
		SceneModifier.PreTickFunc =
			[&BodyHandle2](const FRigidContextSimRW& Context)
			{
				CHECK(BodyHandle2.Pin(Context));
				if (TRigidBodyPtr<FRigidContextSimRW> BodyPtr = BodyHandle2.Pin(Context))
				{
					// Make sure this is a PT body
					REQUIRE(BodyPtr->GetTypeId() == FRigidBodyAsyncPT::GetStaticTypeId());

					BodyPtr->InitTransform(FTransform(FVector(100.0, 0.0, 0.0)));
					CHECK(BodyPtr->GetTransform().GetLocation().X == 100.0);
				}
			};

		// Create an async physics scene
		FRigidFactoryAsync SceneFactory;
		FRigidSceneSettingsAsync SceneSettings;
		FRigidSceneHandle SceneHandle = SceneFactory.CreateScene(FRigidDebugName::Make(TEXT("Main")), &SceneSettings);

		if (FRigidContextGameRW Context = SceneHandle.LockRW())
		{
			Context->RegisterModifier(&SceneModifier);

			TRigidBodyPtr<FRigidContextGameRW> BodyPtr1 = Context->CreateBody(FRigidDebugName::Make(TEXT("Body1")), ERigidMovementType::Dynamic);
			BodyPtr1->CreateShape(MakeBoxShape(FVector3f(100, 100, 100)));
			BodyPtr1->Activate();
			BodyHandle1 = BodyPtr1;

			TRigidBodyPtr<FRigidContextGameRW> BodyPtr2 = Context->CreateBody(FRigidDebugName::Make(TEXT("Body2")), ERigidMovementType::Dynamic);
			BodyPtr2->CreateShape(MakeBoxShape(FVector3f(100, 100, 100)));
			BodyPtr2->Activate();
			BodyHandle2 = BodyPtr2;

			CHECK(BodyPtr1->GetTransform().GetLocation().X == 0.0);
			CHECK(BodyPtr2->GetTransform().GetLocation().X == 0.0);

			Context->StartTick(Dt);
			Context->EndTick();

			// Both bodies were moved in callbacks
			CHECK(BodyPtr1->GetTransform().GetLocation().X == 200.0);
			CHECK(BodyPtr2->GetTransform().GetLocation().X == 100.0);

			Context->DestroyBody(MoveTemp(BodyPtr1));
			Context->DestroyBody(MoveTemp(BodyPtr2));
		}

		// Destroy the scene
		SceneFactory.DestroyScene(SceneHandle);
	}


	// Fake settings class to test scene creation failure
	class FTestSceneSettings : public IRigidSceneSettings
	{
	public:
		UE_RIGIDPHYSICS_RIGIDTYPED_DECL(CHAOSRIGIDPHYSICSASYNC_API, FTestSceneSettings);
	};
	UE_RIGIDPHYSICS_RIGIDTYPED_IMPL(FTestSceneSettings, IRigidSceneSettings);

	// Attempt to create an async scene with settings for a different scene type. Should fail gracefully.
	TEST_CASE("WrongSceneSettings", "[Chaos][API][Scene][unit]")
	{
		// Suppress the error for horde.
		FErrorLogSuppressor SuppressExpectedError;
		SuppressExpectedError.ExpectError(TEXT("FRigidFactoryAsync::CreateScene: Cannot create scene"));

		FRigidFactoryAsync SceneFactory;
		FTestSceneSettings SceneSettings;
		FRigidSceneHandle SceneHandle = SceneFactory.CreateScene(FRigidDebugName::Make(TEXT("Main")), &SceneSettings);
		CHECK_FALSE(SceneHandle.LockRO());
	}


	//
	// NOTE: NOT A REAL TEST
	// This code is just sample code and mainly used to cut and paste images into the architecture presentation!
	//
	TEST_CASE("SceneExampleCode", "![SampleCodeOnly]")
	{
		FRigidFactoryAsync SceneFactory;
		FRigidSceneSettingsAsync SceneSettings;
		FRigidSceneHandle SceneHandle = SceneFactory.CreateScene(FRigidDebugName::Make(TEXT("Main")), &SceneSettings);

		FRigidBodyHandle BodyHandle;
		if (FRigidContextGameRW Context = SceneHandle.LockRW())
		{
			if (TRigidBodyPtr<FRigidContextGameRW> BodyPtr = Context->CreateBody(FRigidDebugName("Body2"), ERigidMovementType::Dynamic))
			{
				BodyPtr->CreateShape(MakeBoxShape(FVector3f(100, 100, 100)));
				BodyPtr->SetIsSleeping(false);
				BodyPtr->SetMass(1000);
				BodyPtr->SetInertia(FVector3d(MakeSolidBoxInertia(1000, FVector3f(100, 100, 100))));
				BodyPtr->InitTransform(FTransform(FVector(0, 0, 300)));
				BodyPtr->Activate();

				BodyHandle = BodyPtr;
			}
		}

		if (FRigidContextGameRW Context = SceneHandle.LockRW())
		{
			if (TRigidBodyPtr<FRigidContextGameRW> BodyPtr = BodyHandle.Pin(Context))
			{
				BodyPtr->SetMass(1000);
				BodyPtr->InitTransform(FTransform(FVector(0, 0, 300)));
			}
		}

		//FPhysicsActorHandle ActorHandle;
		//FPhysicsCommand::ExecuteWrite(ActorHandle,
		//	[](const FPhysicsActorHandle& Actor)
		//	{
		//		FPhysicsInterface::SetGlobalPose_AssumesLocked(Actor, FTransform(FVector(0, 0, 300)));
		//		FPhysicsInterface::SetMass_AssumesLocked(Actor, 1000);
		//	});

		const auto& PTCallback = [&BodyHandle](const FRigidContextSimRW& Context)
			{
				if (TRigidBodyPtr<FRigidContextSimRW> BodyPtr = BodyHandle.Pin(Context))
				{
					BodyPtr->SetMass(1000);
					BodyPtr->InitTransform(FTransform(FVector(0, 0, 300)));
				}
			};

		if (FRigidContextGameRW Context = SceneHandle.LockRW())
		{
			Context->DestroyBody(BodyHandle.Pin(Context));
		}

		SceneFactory.DestroyScene(SceneHandle);
	}

} // Chaos::LowLevelTest

#endif
