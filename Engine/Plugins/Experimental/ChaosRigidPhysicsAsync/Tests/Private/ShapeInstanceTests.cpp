// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/HeightField.h"
#include "Chaos/LowLevelTest/ChaosTestHarness.h"
#include "Chaos/LowLevelTest/ChaosTestScene.h"
#include "ChaosRigidPhysicsAsyncTest.h"
#include "RigidPhysics/RigidBody.h"
#include "RigidPhysics/RigidContext.h"
#include "RigidPhysics/RigidScene.h"
#include "RigidPhysics/RigidShapeInstance.h"
#include "RigidTestFixture.h"
#include "RigidTestUtils.h"
#include <catch2/generators/catch_generators_range.hpp>

#if UE_RIGIDPHYSICS_API_ENABLED

namespace Chaos::LowLevelTest
{
	using namespace UE::Physics;

	template <typename GeometryType>
	void CheckGeometry(const GeometryType& Expected, const GeometryType& Actual)
	{
		CHECK(Expected == Actual);
	}

	void CheckGeometry(const FBoxGeometry& Expected, const FBoxGeometry& Actual)
	{
		CHECK_THAT(Chaos::FVec3f(Expected.GetExtents()), Catch::ApproxEq(Chaos::FVec3f(Actual.GetExtents()), UE_KINDA_SMALL_NUMBER));
	}
	
	void CheckGeometry(const FSphereGeometry& Expected, const FSphereGeometry& Actual)
	{
		CHECK_THAT(Expected.GetRadius(), Catch::Matchers::WithinAbs(Actual.GetRadius(), UE_KINDA_SMALL_NUMBER));
	}

	void CheckGeometry(const FCapsuleGeometry& Expected, const FCapsuleGeometry& Actual)
	{
		CHECK_THAT(Expected.GetRadius(), Catch::Matchers::WithinAbs(Actual.GetRadius(), UE_KINDA_SMALL_NUMBER));
		CHECK_THAT(Expected.GetSegmentHalfLength(), Catch::Matchers::WithinAbs(Actual.GetSegmentHalfLength(), UE_KINDA_SMALL_NUMBER));
	}

	template <typename GeometryType>
	void CheckGeometryTransform(const FTransform3f& Expected, const FTransform3f& Actual)
	{
		// TODO: The Catch2 StringMaker isn't working correctly for FVector3f for some reason. Manually casting to Chaos::FVec3f for now...
		CHECK_THAT(Chaos::FVec3f(Expected.GetLocation()), Catch::ApproxEq(Chaos::FVec3f(Actual.GetLocation())));
		CHECK_THAT(Expected.GetRotation(), Catch::ApproxEq(Actual.GetRotation()));
		CHECK_THAT(Chaos::FVec3f(Expected.GetScale3D()), Catch::ApproxEq(Chaos::FVec3f(Actual.GetScale3D())));
	}

	template <>
	void CheckGeometryTransform<FCapsuleGeometry>(const FTransform3f& Expected, const FTransform3f& Actual)
	{
		CHECK_THAT(Chaos::FVec3f(Expected.GetLocation()), Catch::ApproxEq(Chaos::FVec3f(Actual.GetLocation())));
		CHECK_THAT(Chaos::FVec3f(Expected.GetScale3D()), Catch::ApproxEq(Chaos::FVec3f(Actual.GetScale3D())));

		// Capsule rotations are deconstructed from an axis. There is a many to one mapping for a capsule axis, so we cannot check the rotation directly.
		// Instead, we want to verify the "pose" of the capsule, we can do this by transforming a capsule axis and verifying it's the same.
		const FVec3f ExpectedAxis = Expected.GetRotation().RotateVector(FVec3f(0, 0, 1)).GetSafeNormal();
		FVec3f ActualAxis = Actual.GetRotation().RotateVector(FVec3f(0, 0, 1)).GetSafeNormal();

		constexpr float AngularErrorThreshold = FMath::DegreesToRadians(1);
		float AngularDistanceRad = FMath::Abs(FMath::Acos(ExpectedAxis.Dot(ActualAxis)));
		CHECK(AngularDistanceRad < AngularErrorThreshold);
	}

	struct FShapeInstanceTestFixture : public FRigidTestFixture
	{
		FRigidShapeInstanceHandle CreateBodyAndShapeInstance(const FRigidShapeInstanceSetup& InSetup)
		{
			FRigidShapeInstanceHandle ShapeInstanceHandle;
			if (FRigidContextGameRW Context = LockRWChecked())
			{
				TRigidBodyPtr<FRigidContextGameRW> BodyPtr = Context->CreateBody(FRigidDebugName("Body"), ERigidMovementType::Dynamic);
				AutoCleanup(BodyPtr);

				ShapeInstanceHandle = BodyPtr->CreateShape(InSetup);
				BodyPtr->Activate();
			}
			return ShapeInstanceHandle;
		}
	};

	void CreateAndCheckShapeInstanceAndTransform(FShapeInstanceTestFixture& Fixture, const FRigidShapeInstanceSetup& InSetup, const FAnyGeometry& ExpectedGeometry, const FTransform3f& ExpectedTransform)
	{
		FRigidShapeInstanceHandle ShapeInstanceHandle = Fixture.CreateBodyAndShapeInstance(InSetup);
		if (FRigidContextGameRO Context = Fixture.LockROChecked())
		{
			const TRigidShapeInstancePtr<FRigidContextGameRO> ShapeInstancePtr = ShapeInstanceHandle.Pin(Context);

			const FTransform3f ActualTransform = ShapeInstancePtr->GetLocalTransform();
			const FAnyGeometry ActualGeometry = ShapeInstancePtr->GetGeometry();

			ActualGeometry.Visit([&ExpectedTransform, &ActualTransform, &ExpectedGeometry]<typename GeometryType>(const GeometryType& ActualTypedGeometry)
			{
				const GeometryType* ExpectedTypedGeometry = ExpectedGeometry.TryGet<GeometryType>();
				CHECKED_IF(ExpectedTypedGeometry)
				{
					CheckGeometry(*ExpectedTypedGeometry, ActualTypedGeometry);
				}
				CheckGeometryTransform<GeometryType>(ExpectedTransform, ActualTransform);
			});
		}
	}

	void CreateAndCheckShapeInstanceAndTransform(FShapeInstanceTestFixture& Fixture, const FRigidShapeInstanceSetup& InSetup)
	{
		CreateAndCheckShapeInstanceAndTransform(Fixture, InSetup, InSetup.Geometry, InSetup.LocalTransform);
	}

	TEST_CASE("ShapeInstance: Setup + GetShape/Transform", "[Chaos][API][ShapeInstance][unit]")
	{
		FShapeInstanceTestFixture Fixture;
		SECTION("Box")
		{
			FRigidShapeInstanceSetup Setup(FBoxGeometry(FVector3f(1, 2, 3)));

			SECTION("Identity Transform")
			{
				Setup.LocalTransform.SetIdentity();
				CreateAndCheckShapeInstanceAndTransform(Fixture, Setup);
			}
			SECTION("Translation")
			{
				Setup.LocalTransform.SetTranslation(FVector3f(0, 1, 0));
				CreateAndCheckShapeInstanceAndTransform(Fixture, Setup);
			}
			SECTION("Rotation")
			{
				Setup.LocalTransform.SetRotation(FRotator3f(10, 20, 30).Quaternion());
				CreateAndCheckShapeInstanceAndTransform(Fixture, Setup);
			}
			SECTION("Scale")
			{
				struct FBoxScaleCase
				{
					FBoxScaleCase(const FVector3f& InScale, const FVector3f& InExpectedExtents) : Scale(InScale), ExpectedExtents(InExpectedExtents) {}
					FVector3f Scale;
					FVector3f ExpectedExtents;
				};
				const FBoxScaleCase CaseData = GENERATE(
					FBoxScaleCase(FVector3f(2, 3, 4), FVector3f(2, 6, 12)),
					FBoxScaleCase(FVector3f(-2, -3, -4), FVector3f(2, 6, 12)),
					FBoxScaleCase(FVector3f(0, 0, 0), FVector3f(.1f, .2f, .3f))
				);
				CAPTURE(Chaos::FVec3f(CaseData.Scale));
				Setup.LocalTransform.SetScale3D(CaseData.Scale);
				CreateAndCheckShapeInstanceAndTransform(Fixture, Setup, FBoxGeometry(CaseData.ExpectedExtents), FTransform3f());
			}
			SECTION("Full Transform")
			{
				Setup.LocalTransform.SetLocation(FVector3f(-1, -2, -3));
				Setup.LocalTransform.SetRotation(FRotator3f(10, 20, 30).Quaternion());
				Setup.LocalTransform.SetScale3D(FVector3f(2, 3, 4));
				FTransform3f ExpectedTransform = Setup.LocalTransform;
				ExpectedTransform.SetScale3D(FVec3f::One());
				CreateAndCheckShapeInstanceAndTransform(Fixture, Setup, FBoxGeometry(FVector3f(2, 6, 12)), ExpectedTransform);
			}
			SECTION("Zero Extents")
			{
				Setup = FRigidShapeInstanceSetup(FBoxGeometry(FVector3f::Zero()));
				// Note: The code sets half-extents to UE_KINDA_SMALL_NUMBER, so the full extents become 2 times that.
				const FBoxGeometry ExpectedBox(FVector3f(UE_KINDA_SMALL_NUMBER * 2));
				CreateAndCheckShapeInstanceAndTransform(Fixture, Setup, ExpectedBox, FTransform3f());
			}
		}
		SECTION("Sphere")
		{
			FRigidShapeInstanceSetup Setup(FSphereGeometry(2));
			SECTION("Identity Transform")
			{
				Setup.LocalTransform.SetIdentity();
				CreateAndCheckShapeInstanceAndTransform(Fixture, Setup);
			}
			SECTION("Translation")
			{
				Setup.LocalTransform.SetTranslation(FVector3f(0, 1, 0));
				CreateAndCheckShapeInstanceAndTransform(Fixture, Setup);
			}
			SECTION("Rotation")
			{
				Setup.LocalTransform.SetRotation(FRotator3f(10, 20, 30).Quaternion());
				CreateAndCheckShapeInstanceAndTransform(Fixture, Setup, Setup.Geometry, FTransform3f());
			}
			SECTION("Scale")
			{
				struct FSphereScaleCase
				{
					FSphereScaleCase(const FVector3f& InScale, const float InExpectedRadius) : Scale(InScale), ExpectedRadius(InExpectedRadius) {}
					FVector3f Scale;
					float ExpectedRadius;
				};
				const FSphereScaleCase CaseData = GENERATE(
					// Uniform:
					FSphereScaleCase(FVector3f(3), 6),
					// Non uniform (positive): min element is used
					FSphereScaleCase(FVector3f(2, 3, 4), 4),
					FSphereScaleCase(FVector3f(3, 4, 2), 4),
					FSphereScaleCase(FVector3f(4, 2, 3), 4),
					// Zero:
					FSphereScaleCase(FVector3f::Zero(), 0.2f),
					// Negative:
					FSphereScaleCase(FVector3f(-2, -3, -4), 4), // All are negative, picks min of abs(scale)
					FSphereScaleCase(FVector3f(-2, 1, 3), 2) // Picks largest non-negative
				);
				CAPTURE(Chaos::FVec3f(CaseData.Scale));
				Setup.LocalTransform.SetScale3D(CaseData.Scale);
				CreateAndCheckShapeInstanceAndTransform(Fixture, Setup, FSphereGeometry(CaseData.ExpectedRadius), FTransform3f());
			}
			SECTION("Full Transform")
			{
				Setup.LocalTransform.SetLocation(FVector3f(-1, -2, -3));
				Setup.LocalTransform.SetRotation(FRotator3f(10, 20, 30).Quaternion());
				Setup.LocalTransform.SetScale3D(FVector3f(3));
				FTransform3f ExpectedTransform;
				ExpectedTransform.SetLocation(Setup.LocalTransform.GetLocation());
				CreateAndCheckShapeInstanceAndTransform(Fixture, Setup, FSphereGeometry(6), ExpectedTransform);
			}
			SECTION("Zero Radius")
			{
				Setup = FRigidShapeInstanceSetup(FSphereGeometry(0));
				CreateAndCheckShapeInstanceAndTransform(Fixture, Setup, FSphereGeometry(0), FTransform3f());
			}
		}
		SECTION("Capsule")
		{
			FRigidShapeInstanceSetup Setup(FCapsuleGeometry(2, 3));

			SECTION("Identity Transform")
			{
				Setup.LocalTransform.SetIdentity();
				CreateAndCheckShapeInstanceAndTransform(Fixture, Setup);
			}
			SECTION("Translation")
			{
				Setup.LocalTransform.SetTranslation(FVector3f(0, 1, 0));
				CreateAndCheckShapeInstanceAndTransform(Fixture, Setup);
			}
			SECTION("Rotation")
			{
				const float Degrees = GENERATE(90, 45);
				const FVector3f Axis = GENERATE(
					FVector3f(1, 0, 0),
					FVector3f(0, 1, 0),
					FVector3f(0, 0, 1),
					FVector3f(1, 1, 0),
					FVector3f(1, 0, 1),
					FVector3f(0, 1, 1),
					FVector3f(1, 1, 1)
				);
				const FQuat4f Rotation = FQuat4f(Axis, FMath::DegreesToRadians(Degrees));
				CreateAndCheckShapeInstanceAndTransform(Fixture, Setup, Setup.Geometry, Setup.LocalTransform);
			}
			SECTION("Scale")
			{
				struct FCapsuleScaleCase
				{
					FCapsuleScaleCase(const FVector3f& InScale, const FCapsuleGeometry& InGeometry) : Scale(InScale), ExpectedGeometry(InGeometry) {}
					FVector3f Scale;
					FCapsuleGeometry ExpectedGeometry;
				};
				// Capsule scaling is a bit difficult to understand without considering the full picture.
				// When a capsule is scaled, the full extents are what's actually scaled then the capsule is built from that.
				// This can be thought of as scaling the cylinder that fully contains the capsule.
				// So Capsule(Radius, HalfHeight) -> Cylinder(Radius, FullHalfHeight): Cylinder.FullHalfHeight = Capsule.Radius + Cylinder.HalfHeight.
				// So an example:
				// - Capsule(2, 3) with scale (2, 1, 1)
				//   - Capsule(2, 3) -> Cylinder(2, 3 + 2) -> Cylinder(2, 5)
				//   - So Cylinder(2, 5) * Scale(2, 1) -> Cylinder (4, 5)
				//   - Cylinder(4, 5) -> Capsule(4, 5 - 4) -> Capsule(4, 1)
				// Additionally, the radius and the half height are clamped to prevent invalid capsules:
				// - The radius is Clamp(0.1, HalfHeight)
				// - The half height is Clamp(.05, HalfHeight - Radius)
				FCapsuleScaleCase CaseData = GENERATE(
					FCapsuleScaleCase(FVector3f(2, 1, 1), FCapsuleGeometry(4, 1)),
					FCapsuleScaleCase(FVector3f(1, 2, 1), FCapsuleGeometry(4, 1)),
					// Capsule(2, 3) * (1, 1, 2) -> Cylinder(2, 5) * (1, 2) -> Cylinder(2, 10) -> Capsule(2, 10 - 2) -> Capsule(2, 8)
					FCapsuleScaleCase(FVector3f(1, 1, 2), FCapsuleGeometry(2, 8)),
					// Capsule(2, 3) * (2, 2, 2) -> Cylinder(2, 5) * (2, 2) -> Cylinder(4, 10) -> Capsule(4, 10 - 4) -> Capsule(2, 6)
					FCapsuleScaleCase(FVector3f(2, 2, 2), FCapsuleGeometry(4, 6)),
					// Zero:
					FCapsuleScaleCase(FVector3f::Zero(), FCapsuleGeometry(.2f, .3f)),
					// Capsule(2, 3) * (0, 0, 1) -> Cylinder(2, 5) * (0, 1) -> Cylinder(0, 5) -> Cylinder(0.1, 5) -> Capsule(0.1, 5 - 0.1) -> Capsule(0.1, 4.9)
					FCapsuleScaleCase(FVector3f(0, 0, 1), FCapsuleGeometry(.1f, 4.9f)),
					// This case tests radius being clamped to half height. This turns into Capsule(0, 0)
					FCapsuleScaleCase(FVector3f(1, 1, 0), FCapsuleGeometry(0.1f, 0.05f)),
					// Negative:
					FCapsuleScaleCase(FVector3f(-2, -2, -2), FCapsuleGeometry(4, 6)),
					FCapsuleScaleCase(FVector3f(-2, 1, 1), FCapsuleGeometry(4, 1)), // Same as (2, 1, 1)
					FCapsuleScaleCase(FVector3f(1, 1, -2), FCapsuleGeometry(2, 8)), // Same as (1, 1, 2)
					// This case tests radius being clamped to half height:
					// Capsule(2, 3) * (10, 1) -> Cylinder(2, 5) * (10, 1) -> Cylinder(20, 5) -> (Clamp Radius to HalfHeight) -> Cylinder(5, 5) ->Capsule(5, 5 - 5) -> Capsule(5, 0.1)
					FCapsuleScaleCase(FVector3f(10, 1, 1), FCapsuleGeometry(5, 0.05f))
				);
				CAPTURE(Chaos::FVec3f(CaseData.Scale));
				Setup.LocalTransform.SetScale3D(CaseData.Scale);
				CreateAndCheckShapeInstanceAndTransform(Fixture, Setup, CaseData.ExpectedGeometry, FTransform3f());
			}
			SECTION("Full Transform")
			{
				Setup.LocalTransform.SetLocation(FVector3f(-1, -2, -3));
				Setup.LocalTransform.SetRotation(FRotator3f(10, 20, 30).Quaternion());
				// Capsule(2, 3) * (2, 3, 4) -> Cylinder(2, 5) * (3, 4) -> Cylinder(6, 20) -> Capsule(6, 20 - 6) -> Capsule(6, 14)
				Setup.LocalTransform.SetScale3D(FVector3f(2, 3, 4));
				FTransform3f ExpectedTransform = Setup.LocalTransform;
				ExpectedTransform.SetScale3D(FVec3f::One());
				CreateAndCheckShapeInstanceAndTransform(Fixture, Setup, FCapsuleGeometry(6, 14), ExpectedTransform);
			}
			SECTION("Zero Input")
			{
				Setup = FRigidShapeInstanceSetup(FCapsuleGeometry(0, 0));
				// Internals currently clamp radius and height to 0.1.
				const FCapsuleGeometry Expected(0.1f, 0.05f);
				CreateAndCheckShapeInstanceAndTransform(Fixture, Setup, Expected, FTransform3f());
			}
		}
		SECTION("Convex")
		{
			FConvexGeometry Geometry(UE::Physics::MakeConvexBoxGeometrySetup());
			FRigidShapeInstanceSetup Setup(Geometry);

			SECTION("Basic Transforms")
			{
				const FTransform3f LocalTransform = GENERATE(
					FTransform3f(),
					FTransform3f(FVector3f(1, 2, 3)),
					FTransform3f(FRotator3f(10, 20, 30)),
					FTransform3f(FQuat4f::Identity, FVector3f::Zero(), FVector3f(2)),
					FTransform3f(FQuat4f::Identity, FVector3f::Zero(), FVector3f(2, 3, 4)),
					FTransform3f(FRotator3f(10, 20, 30), FVector3f(1, 2, 3), FVector3f(2, 3, 4)),
					// Negative scales
					FTransform3f(FQuat4f::Identity, FVector3f::Zero(), FVector3f(-2, -3, -4)),
					FTransform3f(FQuat4f::Identity, FVector3f::Zero(), FVector3f(-2, 1, 1))
				);
				Setup.LocalTransform = LocalTransform;
				CreateAndCheckShapeInstanceAndTransform(Fixture, Setup);
			}
			SECTION("Zero Scale")
			{
				Setup.LocalTransform.SetScale3D(FVector3f::Zero());
				const FTransform3f ExpectedTransform(FQuat4f::Identity, FVector3f::Zero(), FVector3f(UE_KINDA_SMALL_NUMBER));
				CreateAndCheckShapeInstanceAndTransform(Fixture, Setup, Setup.Geometry, ExpectedTransform);
			}
		}
		SECTION("TriangleMesh")
		{
			FTriangleMeshGeometry TriMeshGeometry(UE::Physics::MakeQuadTriangleGeometrySetup());
			FRigidShapeInstanceSetup Setup(TriMeshGeometry);
			const FTransform3f LocalTransform = GENERATE(
				FTransform3f(),
				FTransform3f(FVector3f(0, 1, 0)),
				FTransform3f(FRotator3f(10, 20, 30)),
				FTransform3f(FRotator3f(), FVector3f(1, 1, 1), FVector3f(0.5f)),
				FTransform3f(FRotator3f(), FVector3f(1, 1, 1), FVector3f(2)),
				FTransform3f(FRotator3f(10, 20, 30), FVector3f(0, 1, 0), FVector3f(2, 3, 4))
			);
			Setup.LocalTransform = LocalTransform;
			CreateAndCheckShapeInstanceAndTransform(Fixture, Setup);
		}
		SECTION("HeightField")
		{
			FHeightFieldGeometry HeightFieldGeom(UE::Physics::MakeHeightFieldGeometrySetup());
			FRigidShapeInstanceSetup Setup(HeightFieldGeom);
			const FTransform3f LocalTransform = GENERATE(
				FTransform3f(),
				FTransform3f(FVector3f(0, 1, 0)),
				FTransform3f(FRotator3f(10, 20, 30)),
				FTransform3f(FRotator3f(), FVector3f(1, 1, 1), FVector3f(0.5f)),
				FTransform3f(FRotator3f(), FVector3f(1, 1, 1), FVector3f(2)),
				FTransform3f(FRotator3f(10, 20, 30), FVector3f(0, 1, 0), FVector3f(2, 3, 4))
			);
			Setup.LocalTransform = LocalTransform;
			CreateAndCheckShapeInstanceAndTransform(Fixture, Setup);
		}
	}

	TEST_CASE("ShapeInstance: QueryEnabled", "[Chaos][API][ShapeInstance][unit]")
	{
		FShapeInstanceTestFixture Fixture;
		FRigidShapeInstanceSetup Setup(FSphereGeometry(1));

		const bool bQueryEnabled = GENERATE(true, false);
		Setup.bEnableQuery = bQueryEnabled;

		FRigidShapeInstanceHandle ShapeInstanceHandle = Fixture.CreateBodyAndShapeInstance(Setup);
		if (FRigidContextGameRO Context = Fixture.LockROChecked())
		{
			const TRigidShapeInstancePtr<FRigidContextGameRO> ShapeInstancePtr = ShapeInstanceHandle.Pin(Context);
			CHECK(ShapeInstancePtr->GetQueryEnabled() == bQueryEnabled);
		}

		Fixture.RunPTCallback([ShapeInstanceHandle, bQueryEnabled](const FRigidContextSimRW& Context)
			{
				const TRigidShapeInstancePtr<FRigidContextSimRW> ShapeInstancePtr = ShapeInstanceHandle.Pin(Context);
				CHECK(ShapeInstancePtr->GetQueryEnabled() == bQueryEnabled);
			});

		const bool bNewQueryEnabled = !bQueryEnabled;
		if (FRigidContextGameRW Context = Fixture.LockRWChecked())
		{
			TRigidShapeInstancePtr<FRigidContextGameRW> ShapeInstancePtr = ShapeInstanceHandle.Pin(Context);
			ShapeInstancePtr->SetQueryEnabled(bNewQueryEnabled);
			CHECK(ShapeInstancePtr->GetQueryEnabled() == bNewQueryEnabled);
		}

		Fixture.RunPTCallback([ShapeInstanceHandle, bNewQueryEnabled](const FRigidContextSimRW& Context)
			{
				const TRigidShapeInstancePtr<FRigidContextSimRW> ShapeInstancePtr = ShapeInstanceHandle.Pin(Context);
				CHECK(ShapeInstancePtr->GetQueryEnabled() == bNewQueryEnabled);
			});
	}

	TEST_CASE("ShapeInstance: SimEnabled", "[Chaos][API][ShapeInstance][unit]")
	{
		FShapeInstanceTestFixture Fixture;
		FRigidShapeInstanceSetup Setup(FSphereGeometry(1));

		const bool bSimEnabled = GENERATE(true, false);
		Setup.bEnableSim = bSimEnabled;

		FRigidShapeInstanceHandle ShapeInstanceHandle = Fixture.CreateBodyAndShapeInstance(Setup);
		if (FRigidContextGameRO Context = Fixture.LockROChecked())
		{
			const TRigidShapeInstancePtr<FRigidContextGameRO> ShapeInstancePtr = ShapeInstanceHandle.Pin(Context);
			CHECK(ShapeInstancePtr->GetSimEnabled() == bSimEnabled);
		}

		Fixture.RunPTCallback([ShapeInstanceHandle, bSimEnabled](const FRigidContextSimRW& Context)
			{
				const TRigidShapeInstancePtr<FRigidContextSimRW> ShapeInstancePtr = ShapeInstanceHandle.Pin(Context);
				CHECK(ShapeInstancePtr->GetSimEnabled() == bSimEnabled);
			});

		const bool bNewSimEnabled = !bSimEnabled;
		if (FRigidContextGameRW Context = Fixture.LockRWChecked())
		{
			TRigidShapeInstancePtr<FRigidContextGameRW> ShapeInstancePtr = ShapeInstanceHandle.Pin(Context);
			ShapeInstancePtr->SetSimEnabled(bNewSimEnabled);
			CHECK(ShapeInstancePtr->GetSimEnabled() == bNewSimEnabled);
		}

		Fixture.RunPTCallback([ShapeInstanceHandle, bNewSimEnabled](const FRigidContextSimRW& Context)
			{
				const TRigidShapeInstancePtr<FRigidContextSimRW> ShapeInstancePtr = ShapeInstanceHandle.Pin(Context);
				CHECK(ShapeInstancePtr->GetSimEnabled() == bNewSimEnabled);
			});
	}

	TEST_CASE("ShapeInstance: IsProbe", "[Chaos][API][ShapeInstance][unit]")
	{
		FShapeInstanceTestFixture Fixture;
		FRigidShapeInstanceSetup Setup(FSphereGeometry(1));

		const bool bIsProbe = GENERATE(false, true);
		Setup.bEnableProbe = bIsProbe;

		FRigidShapeInstanceHandle ShapeInstanceHandle = Fixture.CreateBodyAndShapeInstance(Setup);
		if (FRigidContextGameRO Context = Fixture.LockROChecked())
		{
			const TRigidShapeInstancePtr<FRigidContextGameRO> ShapeInstancePtr = ShapeInstanceHandle.Pin(Context);
			CHECK(ShapeInstancePtr->GetIsProbe() == bIsProbe);
		}

		Fixture.RunPTCallback([ShapeInstanceHandle, bIsProbe](const FRigidContextSimRW& Context)
			{
				const TRigidShapeInstancePtr<FRigidContextSimRW> ShapeInstancePtr = ShapeInstanceHandle.Pin(Context);
				CHECK(ShapeInstancePtr->GetIsProbe() == bIsProbe);
			});

		const bool bNewIsProbe = !bIsProbe;
		if (FRigidContextGameRW Context = Fixture.LockRWChecked())
		{
			TRigidShapeInstancePtr<FRigidContextGameRW> ShapeInstancePtr = ShapeInstanceHandle.Pin(Context);
			ShapeInstancePtr->SetIsProbe(bNewIsProbe);
			CHECK(ShapeInstancePtr->GetIsProbe() == bNewIsProbe);
		}

		Fixture.RunPTCallback([ShapeInstanceHandle, bNewIsProbe](const FRigidContextSimRW& Context)
			{
				const TRigidShapeInstancePtr<FRigidContextSimRW> ShapeInstancePtr = ShapeInstanceHandle.Pin(Context);
				CHECK(ShapeInstancePtr->GetIsProbe() == bNewIsProbe);
			});
	}

	TEST_CASE("ShapeInstance: CollisionTraceType", "[Chaos][API][ShapeInstance][unit]")
	{
		FShapeInstanceTestFixture Fixture;
		const TArray<ECollisionTraceFlag> TraceTypes
		{
			ECollisionTraceFlag::CTF_UseDefault,
			ECollisionTraceFlag::CTF_UseSimpleAndComplex,
			ECollisionTraceFlag::CTF_UseSimpleAsComplex,
			ECollisionTraceFlag::CTF_UseComplexAsSimple,
		};
		const uint32 Index = GENERATE_COPY(Catch::Generators::range(0, TraceTypes.Num()));
		const ECollisionTraceFlag TraceType = TraceTypes[Index];
		const ECollisionTraceFlag NewTraceType = TraceTypes[(Index + 1) % TraceTypes.Num()];

		FRigidShapeInstanceSetup Setup(FSphereGeometry(1));
		Setup.CollisionTraceType = TraceType;

		FRigidShapeInstanceHandle ShapeInstanceHandle = Fixture.CreateBodyAndShapeInstance(Setup);
		if (FRigidContextGameRO Context = Fixture.LockROChecked())
		{
			const TRigidShapeInstancePtr<FRigidContextGameRO> ShapeInstancePtr = ShapeInstanceHandle.Pin(Context);
			CHECK(ShapeInstancePtr->GetCollisionTraceType() == TraceType);
		}

		Fixture.RunPTCallback([ShapeInstanceHandle, TraceType](const FRigidContextSimRW& Context)
			{
				const TRigidShapeInstancePtr<FRigidContextSimRW> ShapeInstancePtr = ShapeInstanceHandle.Pin(Context);
				CHECK(ShapeInstancePtr->GetCollisionTraceType() == TraceType);
			});

		if (FRigidContextGameRW Context = Fixture.LockRWChecked())
		{
			TRigidShapeInstancePtr<FRigidContextGameRW> ShapeInstancePtr = ShapeInstanceHandle.Pin(Context);
			ShapeInstancePtr->SetCollisionTraceType(NewTraceType);
			CHECK(ShapeInstancePtr->GetCollisionTraceType() == NewTraceType);
		}

		Fixture.RunPTCallback([ShapeInstanceHandle, NewTraceType](const FRigidContextSimRW& Context)
			{
				const TRigidShapeInstancePtr<FRigidContextSimRW> ShapeInstancePtr = ShapeInstanceHandle.Pin(Context);
				CHECK(ShapeInstancePtr->GetCollisionTraceType() == NewTraceType);
			});
	}

	TEST_CASE("ShapeInstance: ShapeFilter", "[Chaos][API][ShapeInstance][unit]")
	{
		FShapeInstanceTestFixture Fixture;

		const TStaticArray<Chaos::Filter::FShapeFilterData, 2> ShapeFilters
		{
			Chaos::Filter::FShapeFilterBuilder().SetCollisionChannelIndex(1).Build(),
			Chaos::Filter::FShapeFilterBuilder().SetCollisionChannelIndex(2).Build(),
		};

		FRigidShapeInstanceSetup Setup(FSphereGeometry(1));
		Setup.ShapeFilterData = ShapeFilters[0];

		FRigidShapeInstanceHandle ShapeInstanceHandle = Fixture.CreateBodyAndShapeInstance(Setup);
		if (FRigidContextGameRO Context = Fixture.LockROChecked())
		{
			const TRigidShapeInstancePtr<FRigidContextGameRO> ShapeInstancePtr = ShapeInstanceHandle.Pin(Context);
			CHECK(ShapeInstancePtr->GetShapeFilter() == ShapeFilters[0]);
		}

		Fixture.RunPTCallback([ShapeInstanceHandle, &ShapeFilters](const FRigidContextSimRW& Context)
			{
				const TRigidShapeInstancePtr<FRigidContextSimRW> ShapeInstancePtr = ShapeInstanceHandle.Pin(Context);
				CHECK(ShapeInstancePtr->GetShapeFilter() == ShapeFilters[0]);
			});

		if (FRigidContextGameRW Context = Fixture.LockRWChecked())
		{
			TRigidShapeInstancePtr<FRigidContextGameRW> ShapeInstancePtr = ShapeInstanceHandle.Pin(Context);
			ShapeInstancePtr->SetShapeFilter(ShapeFilters[1]);
			CHECK(ShapeInstancePtr->GetShapeFilter() == ShapeFilters[1]);
		}

		Fixture.RunPTCallback([ShapeInstanceHandle, &ShapeFilters](const FRigidContextSimRW& Context)
			{
				const TRigidShapeInstancePtr<FRigidContextSimRW> ShapeInstancePtr = ShapeInstanceHandle.Pin(Context);
				CHECK(ShapeInstancePtr->GetShapeFilter() == ShapeFilters[1]);
			});
	}

	TEST_CASE("ShapeInstance: FilterInstanceData", "[Chaos][API][ShapeInstance][unit]")
	{
		FShapeInstanceTestFixture Fixture;

		const TStaticArray<Chaos::Filter::FInstanceData, 2> InstanceData
		{
			Chaos::Filter::FInstanceData(123, 456),
			Chaos::Filter::FInstanceData(789, 1011),
		};

		FRigidShapeInstanceSetup Setup(FSphereGeometry(1));
		Setup.FilterInstanceData = InstanceData[0];

		FRigidShapeInstanceHandle ShapeInstanceHandle = Fixture.CreateBodyAndShapeInstance(Setup);
		if (FRigidContextGameRO Context = Fixture.LockROChecked())
		{
			const TRigidShapeInstancePtr<FRigidContextGameRO> ShapeInstancePtr = ShapeInstanceHandle.Pin(Context);
			CHECK(ShapeInstancePtr->GetFilterInstanceData() == InstanceData[0]);
		}

		Fixture.RunPTCallback([ShapeInstanceHandle, &InstanceData](const FRigidContextSimRW& Context)
			{
				const TRigidShapeInstancePtr<FRigidContextSimRW> ShapeInstancePtr = ShapeInstanceHandle.Pin(Context);
				CHECK(ShapeInstancePtr->GetFilterInstanceData() == InstanceData[0]);
			});

		if (FRigidContextGameRW Context = Fixture.LockRWChecked())
		{
			TRigidShapeInstancePtr<FRigidContextGameRW> ShapeInstancePtr = ShapeInstanceHandle.Pin(Context);
			ShapeInstancePtr->SetFilterInstanceData(InstanceData[1]);
			CHECK(ShapeInstancePtr->GetFilterInstanceData() == InstanceData[1]);
		}

		Fixture.RunPTCallback([ShapeInstanceHandle, &InstanceData](const FRigidContextSimRW& Context)
			{
				const TRigidShapeInstancePtr<FRigidContextSimRW> ShapeInstancePtr = ShapeInstanceHandle.Pin(Context);
				CHECK(ShapeInstancePtr->GetFilterInstanceData() == InstanceData[1]);
			});
	}

	TEST_CASE("ShapeInstance: Materials", "[Chaos][API][ShapeInstance][unit]")
	{
		FShapeInstanceTestFixture Fixture;

		TArray<FMaterialHandle> Materials
		{
			Fixture.CreateMaterial(),
		};

		FTriangleMeshGeometry Geometry(UE::Physics::MakeQuadTriangleGeometrySetup());
		FRigidShapeInstanceSetup Setup(Geometry);
		Setup.Materials = Materials;

		FRigidShapeInstanceHandle ShapeInstanceHandle = Fixture.CreateBodyAndShapeInstance(Setup);
		auto CheckMaterialData = [ShapeInstanceHandle, &Materials]<typename ContextType>(const ContextType& Context)
		{
			TRigidShapeInstancePtr<ContextType> ShapeInstance = ShapeInstanceHandle.Pin(Context);
			REQUIRE(ShapeInstance.IsValid());
			REQUIRE(Materials.Num() == ShapeInstance->GetNumMaterials());
			for (int32 I = 0; I < Materials.Num(); ++I)
			{
				CHECK(Materials[I] == ShapeInstance->GetMaterial(I));
			}
		};

		if (FRigidContextGameRO Context = Fixture.LockROChecked())
		{
			CheckMaterialData(Context);
		}
		Fixture.RunPTCallback(CheckMaterialData);

		// Now check the setter
		Materials = TArray<FMaterialHandle>
		{
			Fixture.CreateMaterial(),
			Fixture.CreateMaterial(),
		};
		if (FRigidContextGameRW Context = Fixture.LockRWChecked())
		{
			TRigidShapeInstancePtr<FRigidContextGameRW> ShapeInstance = ShapeInstanceHandle.Pin(Context);
			ShapeInstance->SetMaterials(TArray(Materials));
		}

		if (FRigidContextGameRO Context = Fixture.LockROChecked())
		{
			CheckMaterialData(Context);
		}
		Fixture.RunPTCallback(CheckMaterialData);
	}

	TEST_CASE("ShapeInstance: MaterialMasks", "[Chaos][API][ShapeInstance][unit]")
	{
		FShapeInstanceTestFixture Fixture;

		TArray<FMaterialMaskHandle> MaterialMasks
		{
			Fixture.CreateMaterialMask(),
		};

		FTriangleMeshGeometry Geometry(UE::Physics::MakeQuadTriangleGeometrySetup());
		FRigidShapeInstanceSetup Setup(Geometry);
		Setup.MaterialMaskHandles = MaterialMasks;

		FRigidShapeInstanceHandle ShapeInstanceHandle = Fixture.CreateBodyAndShapeInstance(Setup);
		auto CheckMaterialMaskData = [ShapeInstanceHandle, &MaterialMasks]<typename ContextType>(const ContextType& Context)
		{
			TRigidShapeInstancePtr<ContextType> ShapeInstance = ShapeInstanceHandle.Pin(Context);
			REQUIRE(ShapeInstance.IsValid());
			REQUIRE(MaterialMasks.Num() == ShapeInstance->GetNumMaterialMasks());
			for (int32 I = 0; I < MaterialMasks.Num(); ++I)
			{
				CHECK(MaterialMasks[I] == ShapeInstance->GetMaterialMask(I));
			}
		};

		if (FRigidContextGameRO Context = Fixture.LockROChecked())
		{
			CheckMaterialMaskData(Context);
		}
		Fixture.RunPTCallback(CheckMaterialMaskData);

		// Now check the setter
		MaterialMasks = TArray<FMaterialMaskHandle>
		{
			Fixture.CreateMaterialMask(),
			Fixture.CreateMaterialMask(),
		};
		if (FRigidContextGameRW Context = Fixture.LockRWChecked())
		{
			TRigidShapeInstancePtr<FRigidContextGameRW> ShapeInstance = ShapeInstanceHandle.Pin(Context);
			ShapeInstance->SetMaterialMasks(TArray(MaterialMasks));
		}

		if (FRigidContextGameRO Context = Fixture.LockROChecked())
		{
			CheckMaterialMaskData(Context);
		}
		Fixture.RunPTCallback(CheckMaterialMaskData);
	}

	TEST_CASE("ShapeInstance: MaterialMaskMaps", "[Chaos][API][ShapeInstance][unit]")
	{
		FShapeInstanceTestFixture Fixture;

		TArray<uint32> MaterialMaskMaps{ 1, 2 };

		FTriangleMeshGeometry Geometry(UE::Physics::MakeQuadTriangleGeometrySetup());
		FRigidShapeInstanceSetup Setup(Geometry);
		Setup.MaterialMaskMaps = MaterialMaskMaps;

		FRigidShapeInstanceHandle ShapeInstanceHandle = Fixture.CreateBodyAndShapeInstance(Setup);
		auto CheckMaterialMaskMapData = [ShapeInstanceHandle, &MaterialMaskMaps]<typename ContextType>(const ContextType& Context)
		{
			TRigidShapeInstancePtr<ContextType> ShapeInstance = ShapeInstanceHandle.Pin(Context);
			REQUIRE(ShapeInstance.IsValid());
			REQUIRE(MaterialMaskMaps.Num() == ShapeInstance->GetNumMaterialMaskMaps());
			for (int32 I = 0; I < MaterialMaskMaps.Num(); ++I)
			{
				CHECK(MaterialMaskMaps[I] == ShapeInstance->GetMaterialMaskMap(I));
			}
		};

		if (FRigidContextGameRO Context = Fixture.LockROChecked())
		{
			CheckMaterialMaskMapData(Context);
		}
		Fixture.RunPTCallback(CheckMaterialMaskMapData);

		// Now check the setter
		MaterialMaskMaps = TArray<uint32>{ 3, 4, 5 };
		if (FRigidContextGameRW Context = Fixture.LockRWChecked())
		{
			TRigidShapeInstancePtr<FRigidContextGameRW> ShapeInstance = ShapeInstanceHandle.Pin(Context);
			ShapeInstance->SetMaterialMaskMaps(TArray(MaterialMaskMaps));
		}

		if (FRigidContextGameRO Context = Fixture.LockROChecked())
		{
			CheckMaterialMaskMapData(Context);
		}
		Fixture.RunPTCallback(CheckMaterialMaskMapData);
	}

	TEST_CASE("ShapeInstance: MaterialMaskMapMaterials", "[Chaos][API][ShapeInstance][unit]")
	{
		FShapeInstanceTestFixture Fixture;

		TArray<FMaterialHandle> MaterialMaskMapMaterials
		{
			Fixture.CreateMaterial(),
		};

		FTriangleMeshGeometry Geometry(UE::Physics::MakeQuadTriangleGeometrySetup());
		FRigidShapeInstanceSetup Setup(Geometry);
		Setup.MaterialMaskMapMaterials = MaterialMaskMapMaterials;

		FRigidShapeInstanceHandle ShapeInstanceHandle = Fixture.CreateBodyAndShapeInstance(Setup);
		auto CheckMaterialMaskMapMaterialData = [ShapeInstanceHandle, &MaterialMaskMapMaterials]<typename ContextType>(const ContextType& Context)
		{
			TRigidShapeInstancePtr<ContextType> ShapeInstance = ShapeInstanceHandle.Pin(Context);
			REQUIRE(ShapeInstance.IsValid());
			REQUIRE(MaterialMaskMapMaterials.Num() == ShapeInstance->GetNumMaterialMaskMapMaterials());
			for (int32 I = 0; I < MaterialMaskMapMaterials.Num(); ++I)
			{
				CHECK(MaterialMaskMapMaterials[I] == ShapeInstance->GetMaterialMaskMapMaterial(I));
			}
		};

		if (FRigidContextGameRO Context = Fixture.LockROChecked())
		{
			CheckMaterialMaskMapMaterialData(Context);
		}
		Fixture.RunPTCallback(CheckMaterialMaskMapMaterialData);

		// Now check the setter
		MaterialMaskMapMaterials = TArray<FMaterialHandle>
		{
			Fixture.CreateMaterial(),
			Fixture.CreateMaterial(),
		};
		if (FRigidContextGameRW Context = Fixture.LockRWChecked())
		{
			TRigidShapeInstancePtr<FRigidContextGameRW> ShapeInstance = ShapeInstanceHandle.Pin(Context);
			ShapeInstance->SetMaterialMaskMapMaterials(TArray(MaterialMaskMapMaterials));
		}

		if (FRigidContextGameRO Context = Fixture.LockROChecked())
		{
			CheckMaterialMaskMapMaterialData(Context);
		}
		Fixture.RunPTCallback(CheckMaterialMaskMapMaterialData);
	}

	TEST_CASE("ShapeInstance: UserData", "[Chaos][API][ShapeInstance][unit]")
	{
		FShapeInstanceTestFixture Fixture;

		TStaticArray<int32, 2> UserData{ 1, 2 };

		FRigidShapeInstanceSetup Setup(FSphereGeometry(1));
		Setup.UserData = &UserData[0];

		FRigidShapeInstanceHandle ShapeInstanceHandle = Fixture.CreateBodyAndShapeInstance(Setup);
		if (FRigidContextGameRO Context = Fixture.LockROChecked())
		{
			const TRigidShapeInstancePtr<FRigidContextGameRO> ShapeInstancePtr = ShapeInstanceHandle.Pin(Context);
			CHECK(ShapeInstancePtr->GetUserData() == &UserData[0]);
		}

		Fixture.RunPTCallback([ShapeInstanceHandle, &UserData](const FRigidContextSimRW& Context)
			{
				const TRigidShapeInstancePtr<FRigidContextSimRW> ShapeInstancePtr = ShapeInstanceHandle.Pin(Context);
				CHECK(ShapeInstancePtr->GetUserData() == &UserData[0]);
			});

		if (FRigidContextGameRW Context = Fixture.LockRWChecked())
		{
			TRigidShapeInstancePtr<FRigidContextGameRW> ShapeInstancePtr = ShapeInstanceHandle.Pin(Context);
			ShapeInstancePtr->SetUserData(&UserData[1]);
			CHECK(ShapeInstancePtr->GetUserData() == &UserData[1]);
		}

		Fixture.RunPTCallback([ShapeInstanceHandle, &UserData](const FRigidContextSimRW& Context)
			{
				const TRigidShapeInstancePtr<FRigidContextSimRW> ShapeInstancePtr = ShapeInstanceHandle.Pin(Context);
				CHECK(ShapeInstancePtr->GetUserData() == &UserData[1]);
			});
	}

	// This test verifies that creating a shape instance from the geometry of another shape instance shares the implicit.
	// This is tied to the current implementation of Chaos and should eventually be updated to reflect the new storage when that's completed.
	TEST_CASE("ShapeInstance: SharedImplicit", "[Chaos][API][ShapeInstance][unit]")
	{
		FShapeInstanceTestFixture Fixture;
		auto TestSharedImplicits = [&Fixture]<typename GeometryType>(const GeometryType& InGeometry)
		{
			FRigidShapeInstanceHandle ShapeHandle0, ShapeHandle1;
			if (FRigidContextGameRW Context = Fixture.LockRWChecked())
			{
				TRigidBodyPtr<FRigidContextGameRW> BodyPtr = Context->CreateBody(FRigidDebugName("Body1"), ERigidMovementType::Dynamic);
				Fixture.AutoCleanup(BodyPtr);

				FRigidShapeInstanceSetup Setup0(InGeometry);
				TRigidShapeInstancePtr<FRigidContextGameRW> ShapeInstance0 = BodyPtr->CreateShape(Setup0);
				ShapeHandle0 = ShapeInstance0;

				// Create another shape instance with the same geometry but some different instance data
				FRigidShapeInstanceSetup Setup1(ShapeInstance0->GetGeometry());
				Setup1.LocalTransform.SetTranslation(FVector3f(1, 2, 3));
				ShapeHandle1 = BodyPtr->CreateShape(Setup1);
				BodyPtr->Activate();
			}

			auto CheckSharedImplicits = [&Fixture, &ShapeHandle0, &ShapeHandle1]<typename ContextType>(const ContextType& Context)
			{
				TRigidShapeInstancePtr<ContextType> ShapeInstance0 = ShapeHandle0.Pin(Context);
				TRigidShapeInstancePtr<ContextType> ShapeInstance1 = ShapeHandle1.Pin(Context);
				REQUIRE(ShapeInstance0);
				REQUIRE(ShapeInstance1);

				const FAnyGeometry AnyGeometry0 = ShapeInstance0->GetGeometry();
				const FAnyGeometry AnyGeometry1 = ShapeInstance1->GetGeometry();
				const GeometryType& TypedGeometry0 = AnyGeometry0.Get<GeometryType>();
				const GeometryType& TypedGeometry1 = AnyGeometry1.Get<GeometryType>();

				CHECK(TypedGeometry0.GetImplicit() == TypedGeometry1.GetImplicit());
			};

			if (FRigidContextGameRO Context = Fixture.LockROChecked())
			{
				CheckSharedImplicits(Context);
			}
			Fixture.RunPTCallback(CheckSharedImplicits);
		};
		SECTION("Convex")
		{
			FConvexGeometry ConvexGeom(UE::Physics::MakeConvexBoxGeometrySetup());
			TestSharedImplicits(ConvexGeom);
		}
		SECTION("TriangleMesh")
		{
			FTriangleMeshGeometry TriMeshGeometry(UE::Physics::MakeQuadTriangleGeometrySetup());
			TestSharedImplicits(TriMeshGeometry);
		}
		SECTION("HeightField")
		{
			FHeightFieldGeometry HeightFieldGeom(UE::Physics::MakeHeightFieldGeometrySetup());
			TestSharedImplicits(HeightFieldGeom);
		}
	}
} // namespace Chaos::LowLevelTest

#endif // UE_RIGIDPHYSICS_API_ENABLED
