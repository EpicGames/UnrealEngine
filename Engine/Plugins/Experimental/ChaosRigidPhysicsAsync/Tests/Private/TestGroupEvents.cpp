// Copyright Epic Games, Inc. All Rights Reserved.
#include <catch2/catch_test_macros.hpp>

#if UE_RIGIDPHYSICS_API_ENABLED

#include "RigidPhysics/RigidSceneRegistry.h"

GROUP_BEFORE_EACH(Catch::DefaultGroup)
{
	UE::Physics::FRigidSceneRegistry::GetInstance().Reset();
}

#endif // UE_RIGIDPHYSICS_API_ENABLED
