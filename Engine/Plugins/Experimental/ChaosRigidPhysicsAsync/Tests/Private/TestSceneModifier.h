// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "RigidPhysics/RigidModifier.h"

namespace Chaos::LowLevelTest
{
	class FTestRigidSceneModifier : public UE::Physics::IRigidSceneModifier
	{
	public:
		virtual void PreSimulate(const UE::Physics::FRigidContextGameRW& Context) override final
		{
			if (PreSimFunc)
			{
				PreSimFunc(Context);
			}
		}

		virtual void PreTick(const UE::Physics::FRigidContextSimRW& Context) override final
		{
			if (PreTickFunc)
			{
				PreTickFunc(Context);
			}
		}

		TFunction<void(const UE::Physics::FRigidContextGameRW&)> PreSimFunc;
		TFunction<void(const UE::Physics::FRigidContextSimRW&)> PreTickFunc;
	};
} // namespace Chaos::LowLevelTest