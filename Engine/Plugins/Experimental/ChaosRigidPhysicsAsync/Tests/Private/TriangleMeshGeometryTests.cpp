// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/LowLevelTest/ChaosTestHarness.h"
#include "ChaosRigidPhysicsAsyncTest.h"

#include "RigidPhysics/Geometry/TriangleMeshGeometry.h"
#include "RigidTestUtils.h"

#if UE_RIGIDPHYSICS_API_ENABLED

namespace Chaos::LowLevelTest
{
	using FTriangleMeshGeometry = UE::Physics::FTriangleMeshGeometry;
	using FTriangleMeshGeometrySetup = UE::Physics::FTriangleMeshGeometrySetup;

	TEST_CASE("TriangleMeshGeometry::Basic", "[Chaos][API][Geometry][unit]")
	{
		const FTriangleMeshGeometrySetup Setup = GENERATE(
			UE::Physics::MakeSingleTriangleGeometrySetup(),
			UE::Physics::MakeQuadTriangleGeometrySetup()
		);
		FTriangleMeshGeometrySetup MutableSetup = Setup;
		const FTriangleMeshGeometry Geometry(MoveTemp(MutableSetup));

		REQUIRE(Setup.Vertices.Num() == Geometry.GetNumVertices());
		for (int32 I = 0; I < Setup.Vertices.Num(); ++I)
		{
			CHECK(Setup.Vertices[I] == Geometry.GetVertex(I));
		}
		REQUIRE(Setup.TriangleIndices.Num() == Geometry.GetNumTriangleIndices());
		for (int32 I = 0; I < Setup.TriangleIndices.Num(); ++I)
		{
			CHECK(Setup.TriangleIndices[I] == Geometry.GetTriangleIndices(I));
		}
		REQUIRE(Setup.MaterialIndices.Num() == Geometry.GetNumMaterialIndices());
		for (int32 I = 0; I < Setup.MaterialIndices.Num(); ++I)
		{
			CHECK(Setup.MaterialIndices[I] == Geometry.GetMaterialIndex(I));
		}
		REQUIRE(Setup.ExternalFaceIndexMap.IsEmpty() == !Geometry.HasExternalFaceIndices());
		for (int32 I = 0; I < Setup.ExternalFaceIndexMap.Num(); ++I)
		{
			CHECK(Setup.ExternalFaceIndexMap[I] == Geometry.GetExternalFaceIndex(I));
		}
		REQUIRE(Setup.ExternalVertexIndexMap.IsEmpty() == !Geometry.HasExternalVertexIndices());
		for (int32 I = 0; I < Setup.ExternalVertexIndexMap.Num(); ++I)
		{
			CHECK(Setup.ExternalVertexIndexMap[I] == Geometry.GetExternalVertexIndex(I));
		}
		CHECK(Setup.bCullsBackFaceRaycast == Geometry.GetCullsBackFaceRaycast());
		CHECK(Setup.PerFaceBoundsRepresentation == Geometry.GetPerFaceBoundsRepresentation());
	}

	TEST_CASE("TriangleMeshGeometry::ExternalFaceIndexMap", "[Chaos][API][Geometry][unit]")
	{
		FTriangleMeshGeometrySetup Setup = UE::Physics::MakeQuadTriangleGeometrySetup();
		SECTION("Empty")
		{
			const FTriangleMeshGeometry Geometry(MoveTemp(Setup));
			CHECK(!Geometry.HasExternalFaceIndices());
		}
		SECTION("Filled")
		{
			const TArray<int32> ExternalFaceIndices{ 10, 11 };
			Setup.ExternalFaceIndexMap = ExternalFaceIndices;
			const FTriangleMeshGeometry Geometry(MoveTemp(Setup));

			REQUIRE(Geometry.HasExternalFaceIndices());
			for (int32 I = 0; I < ExternalFaceIndices.Num(); ++I)
			{
				CHECK(ExternalFaceIndices[I] == Geometry.GetExternalFaceIndex(I));
			}
		}
	}

	TEST_CASE("TriangleMeshGeometry::ExternalVertexIndexMap", "[Chaos][API][Geometry][unit]")
	{
		FTriangleMeshGeometrySetup Setup = UE::Physics::MakeQuadTriangleGeometrySetup();
		SECTION("Empty")
		{
			const FTriangleMeshGeometry Geometry(MoveTemp(Setup));

			CHECK(!Geometry.HasExternalVertexIndices());
		}
		SECTION("Filled")
		{
			const TArray<int32> ExternalVertexIndices{ 10, 11, 12, 13 };
			Setup.ExternalVertexIndexMap = ExternalVertexIndices;
			const FTriangleMeshGeometry Geometry(MoveTemp(Setup));

			REQUIRE(Geometry.HasExternalVertexIndices());
			for (int32 I = 0; I < ExternalVertexIndices.Num(); ++I)
			{
				CHECK(ExternalVertexIndices[I] == Geometry.GetExternalVertexIndex(I));
			}
		}
	}

	TEST_CASE("TriangleMeshGeometry::CullsBackFaceRaycast", "[Chaos][API][Geometry][unit]")
	{
		const bool bCullsBackFaceRaycast = GENERATE(false, true);

		FTriangleMeshGeometrySetup Setup = UE::Physics::MakeSingleTriangleGeometrySetup();
		Setup.bCullsBackFaceRaycast = bCullsBackFaceRaycast;
		const FTriangleMeshGeometry Geometry(MoveTemp(Setup));

		CHECK(Geometry.GetCullsBackFaceRaycast() == bCullsBackFaceRaycast);
	}

	TEST_CASE("TriangleMeshGeometry::PerFaceBoundsRepresentation", "[Chaos][API][Geometry][unit]")
	{
		using EPerFaceBoundsRepresentation = FTriangleMeshGeometrySetup::EPerFaceBoundsRepresentation;
		const EPerFaceBoundsRepresentation PerFaceBoundsRepresentation = GENERATE(EPerFaceBoundsRepresentation::VectorizedFloat, EPerFaceBoundsRepresentation::Byte);

		FTriangleMeshGeometrySetup Setup = UE::Physics::MakeSingleTriangleGeometrySetup();
		Setup.PerFaceBoundsRepresentation = PerFaceBoundsRepresentation;
		const FTriangleMeshGeometry Geometry(MoveTemp(Setup));

		CHECK(Geometry.GetPerFaceBoundsRepresentation() == PerFaceBoundsRepresentation);
	}
} // namespace Chaos::LowLevelTest

#endif // UE_RIGIDPHYSICS_API_ENABLED
