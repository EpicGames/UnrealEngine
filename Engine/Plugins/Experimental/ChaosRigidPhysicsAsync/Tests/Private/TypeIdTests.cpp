// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/Collision/ContactPoint.h"
#include "Chaos/LowLevelTest/ChaosTestHarness.h"
#include "Chaos/LowLevelTest/ChaosTestScene.h"
#include "ChaosRigidPhysicsAsyncTest.h"

#include <catch2/matchers/catch_matchers_string.hpp>

#if UE_RIGIDPHYSICS_API_ENABLED

namespace Chaos::LowLevelTest
{
	using namespace Chaos;
	using namespace UE::Physics;

	class FTestTypeIdBase : public IRigidTyped
	{
	public:
		UE_RIGIDPHYSICS_RIGIDTYPED_DECL(CHAOSRIGIDPHYSICSASYNC_API, FTestTypeIdBase);
	};

	class FTestTypeIdDerived1 : public FTestTypeIdBase
	{
	public:
		UE_RIGIDPHYSICS_RIGIDTYPED_DECL(CHAOSRIGIDPHYSICSASYNC_API, FTestTypeIdDerived1);
	};

	class FTestTypeIdDerived2 : public FTestTypeIdBase
	{
	public:
		UE_RIGIDPHYSICS_RIGIDTYPED_DECL(CHAOSRIGIDPHYSICSASYNC_API, FTestTypeIdDerived2);
	};

	class FTestTypeIdLeaf1A : public FTestTypeIdDerived1
	{
	public:
		UE_RIGIDPHYSICS_RIGIDTYPED_DECL(CHAOSRIGIDPHYSICSASYNC_API, FTestTypeIdLeaf1A);
	};

	class FTestTypeIdLeaf1B : public FTestTypeIdDerived1
	{
	public:
		UE_RIGIDPHYSICS_RIGIDTYPED_DECL(CHAOSRIGIDPHYSICSASYNC_API, FTestTypeIdLeaf1B);
	};

	UE_RIGIDPHYSICS_RIGIDTYPED_IMPL(FTestTypeIdBase, IRigidTyped);
	UE_RIGIDPHYSICS_RIGIDTYPED_IMPL(FTestTypeIdDerived1, FTestTypeIdBase);
	UE_RIGIDPHYSICS_RIGIDTYPED_IMPL(FTestTypeIdDerived2, FTestTypeIdBase);
	UE_RIGIDPHYSICS_RIGIDTYPED_IMPL(FTestTypeIdLeaf1A, FTestTypeIdDerived1);
	UE_RIGIDPHYSICS_RIGIDTYPED_IMPL(FTestTypeIdLeaf1B, FTestTypeIdDerived1);

	// Verify that up/downcasting succeeds and fails as expected
	TEST_CASE("TypeIdTests", "[Chaos][API][TypeId][unit]")
	{
		// Class Hierarchy:
		// IRigidTyped
		//	+- FTestTypeIdBase
		//		+- FTestTypeIdDerived1
		//		| +- FTestTypeIdLeaf1A
		//		| +- FTestTypeIdLeaf1B
		//		+- FTestTypeIdDerived2
		FTestTypeIdBase Base;
		FTestTypeIdDerived1 Derived1;
		FTestTypeIdDerived2 Derived2;
		FTestTypeIdLeaf1A Leaf1A;
		FTestTypeIdLeaf1B Leaf1B;

		const FTestTypeIdBase* BasePtr = &Base;
		const FTestTypeIdBase* Derived1BasePtr = &Derived1;
		const FTestTypeIdBase* Derived2BasePtr = &Derived2;
		const FTestTypeIdBase* Leaf1ABasePtr = &Leaf1A;
		const FTestTypeIdBase* Leaf1BBasePtr = &Leaf1B;

		const FTestTypeIdDerived1* Leaf1ADerived1Ptr = &Leaf1A;

		// Check type names (generated in the macro)
		CHECK_THAT(ToStdString(BasePtr->GetTypeId().GetTypeName()), Catch::Matchers::Equals("FTestTypeIdBase"));
		CHECK_THAT(ToStdString(Derived1BasePtr->GetTypeId().GetTypeName()), Catch::Matchers::Equals("FTestTypeIdDerived1"));
		CHECK_THAT(ToStdString(Derived2BasePtr->GetTypeId().GetTypeName()), Catch::Matchers::Equals("FTestTypeIdDerived2"));
		CHECK_THAT(ToStdString(Leaf1ABasePtr->GetTypeId().GetTypeName()), Catch::Matchers::Equals("FTestTypeIdLeaf1A"));
		CHECK_THAT(ToStdString(Leaf1BBasePtr->GetTypeId().GetTypeName()), Catch::Matchers::Equals("FTestTypeIdLeaf1B"));

		// Type checks from known leaf type
		CHECK(Base.IsA<IRigidTyped>());
		CHECK(Base.IsA<FTestTypeIdBase>());
		CHECK_FALSE(Base.IsA<FTestTypeIdDerived1>());
		CHECK_FALSE(Base.IsA<FTestTypeIdDerived2>());
		CHECK_FALSE(Base.IsA<FTestTypeIdLeaf1A>());
		CHECK_FALSE(Base.IsA<FTestTypeIdLeaf1B>());

		CHECK(Derived1.IsA<IRigidTyped>());
		CHECK(Derived1.IsA<FTestTypeIdBase>());
		CHECK(Derived1.IsA<FTestTypeIdDerived1>());
		CHECK_FALSE(Derived1.IsA<FTestTypeIdDerived2>());
		CHECK_FALSE(Derived1.IsA<FTestTypeIdLeaf1A>());
		CHECK_FALSE(Derived1.IsA<FTestTypeIdLeaf1B>());

		CHECK(Derived2.IsA<IRigidTyped>());
		CHECK(Derived2.IsA<FTestTypeIdBase>());
		CHECK_FALSE(Derived2.IsA<FTestTypeIdDerived1>());
		CHECK(Derived2.IsA<FTestTypeIdDerived2>());
		CHECK_FALSE(Derived2.IsA<FTestTypeIdLeaf1A>());
		CHECK_FALSE(Derived2.IsA<FTestTypeIdLeaf1B>());

		CHECK(Leaf1A.IsA<IRigidTyped>());
		CHECK(Leaf1A.IsA<FTestTypeIdBase>());
		CHECK(Leaf1A.IsA<FTestTypeIdDerived1>());
		CHECK_FALSE(Leaf1A.IsA<FTestTypeIdDerived2>());
		CHECK(Leaf1A.IsA<FTestTypeIdLeaf1A>());
		CHECK_FALSE(Leaf1A.IsA<FTestTypeIdLeaf1B>());

		CHECK(Leaf1B.IsA<IRigidTyped>());
		CHECK(Leaf1B.IsA<FTestTypeIdBase>());
		CHECK(Leaf1B.IsA<FTestTypeIdDerived1>());
		CHECK_FALSE(Leaf1B.IsA<FTestTypeIdDerived2>());
		CHECK_FALSE(Leaf1B.IsA<FTestTypeIdLeaf1A>());
		CHECK(Leaf1B.IsA<FTestTypeIdLeaf1B>());

		// Type checks from base pointer type

		CHECK(BasePtr->IsA<IRigidTyped>());
		CHECK(BasePtr->IsA<FTestTypeIdBase>());
		CHECK_FALSE(BasePtr->IsA<FTestTypeIdDerived1>());
		CHECK_FALSE(BasePtr->IsA<FTestTypeIdDerived2>());
		CHECK_FALSE(BasePtr->IsA<FTestTypeIdLeaf1A>());
		CHECK_FALSE(BasePtr->IsA<FTestTypeIdLeaf1B>());

		CHECK(Derived1BasePtr->IsA<IRigidTyped>());
		CHECK(Derived1BasePtr->IsA<FTestTypeIdBase>());
		CHECK(Derived1BasePtr->IsA<FTestTypeIdDerived1>());
		CHECK_FALSE(Derived1BasePtr->IsA<FTestTypeIdDerived2>());
		CHECK_FALSE(Derived1BasePtr->IsA<FTestTypeIdLeaf1A>());
		CHECK_FALSE(Derived1BasePtr->IsA<FTestTypeIdLeaf1B>());

		CHECK(Derived2BasePtr->IsA<IRigidTyped>());
		CHECK(Derived2BasePtr->IsA<FTestTypeIdBase>());
		CHECK_FALSE(Derived2BasePtr->IsA<FTestTypeIdDerived1>());
		CHECK(Derived2BasePtr->IsA<FTestTypeIdDerived2>());
		CHECK_FALSE(Derived2BasePtr->IsA<FTestTypeIdLeaf1A>());
		CHECK_FALSE(Derived2BasePtr->IsA<FTestTypeIdLeaf1B>());

		CHECK(Leaf1ABasePtr->IsA<IRigidTyped>());
		CHECK(Leaf1ABasePtr->IsA<FTestTypeIdBase>());
		CHECK(Leaf1ABasePtr->IsA<FTestTypeIdDerived1>());
		CHECK_FALSE(Leaf1ABasePtr->IsA<FTestTypeIdDerived2>());
		CHECK(Leaf1ABasePtr->IsA<FTestTypeIdLeaf1A>());
		CHECK_FALSE(Leaf1ABasePtr->IsA<FTestTypeIdLeaf1B>());

		CHECK(Leaf1BBasePtr->IsA<IRigidTyped>());
		CHECK(Leaf1BBasePtr->IsA<FTestTypeIdBase>());
		CHECK(Leaf1BBasePtr->IsA<FTestTypeIdDerived1>());
		CHECK_FALSE(Leaf1BBasePtr->IsA<FTestTypeIdDerived2>());
		CHECK_FALSE(Leaf1BBasePtr->IsA<FTestTypeIdLeaf1A>());
		CHECK(Leaf1BBasePtr->IsA<FTestTypeIdLeaf1B>());

		// Type checks from intermediate pointer
		CHECK(Leaf1ADerived1Ptr->IsA<IRigidTyped>());
		CHECK(Leaf1ADerived1Ptr->IsA<FTestTypeIdBase>());
		CHECK(Leaf1ADerived1Ptr->IsA<FTestTypeIdDerived1>());
		CHECK_FALSE(Leaf1ADerived1Ptr->IsA<FTestTypeIdDerived2>());
		CHECK(Leaf1ADerived1Ptr->IsA<FTestTypeIdLeaf1A>());
		CHECK_FALSE(Leaf1ADerived1Ptr->IsA<FTestTypeIdLeaf1B>());
	}
}

#endif