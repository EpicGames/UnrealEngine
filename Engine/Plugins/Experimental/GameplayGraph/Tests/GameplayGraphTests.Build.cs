// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class GameplayGraphTests : TestModuleRules
{
	static GameplayGraphTests()
	{
		TestMetadata = new Metadata()
		{
			TestName = "GameplayGraph",
			TestShortName = "GameplayGraph"
		};
	}

	public GameplayGraphTests(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"GameplayGraph"
			});
	}
}