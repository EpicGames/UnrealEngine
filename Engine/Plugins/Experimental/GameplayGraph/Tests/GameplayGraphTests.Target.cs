// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedPlatforms(UnrealPlatformClass.All)]
public class GameplayGraphTestsTarget : TestTargetRules
{
	public GameplayGraphTestsTarget(TargetInfo Target) : base(Target)
	{
	}
}
