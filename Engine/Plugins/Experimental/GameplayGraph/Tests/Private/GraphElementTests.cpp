// Copyright Epic Games, Inc. All Rights Reserved.
#include "TestGraphElement.h"
#include "TestHarness.h"

#include "Graph/Graph.h"
#include "Graph/GraphIsland.h"
#include "Graph/GraphVertex.h"

TEST_CASE("Graph::Element::Element Type", "[graph][element]")
{
	SECTION("Vertex")
	{
		UGraphVertex* Vertex = NewObject<UGraphVertex>();
		CHECK(Vertex->GetElementType() == EGraphElementType::Node);
	}

	SECTION("Island")
	{
		UGraphIsland* Island = NewObject<UGraphIsland>();
		CHECK(Island->GetElementType() == EGraphElementType::Island);
	}
}

TEST_CASE("Graph::Element::Unique Index", "[graph][properties]")
{
	UTestGraphElement* Element = NewObject<UTestGraphElement>();
	SECTION("Non-Temp")
	{
		FGraphUniqueIndex Test{ FGuid::NewGuid(), false };
		Element->PublicSetUniqueIndex(Test);
		CHECK(Element->PublicGetUniqueIndex() == Test);
	}

	SECTION("Non-Temp")
	{
		FGraphUniqueIndex Test{ FGuid::NewGuid(), true };
		Element->PublicSetUniqueIndex(Test);
		CHECK(Element->PublicGetUniqueIndex() == Test);
	}
}

TEST_CASE("Graph::Element::Graph Pointer", "[graph][properties]")
{
	UTestGraphElement* Element = NewObject<UTestGraphElement>();

	SECTION("Set graph")
	{
		TObjectPtr<UGraph> TestGraph = NewObject<UGraph>();
		Element->PublicSetParentGraph(TestGraph);
		CHECK(Element->PublicGetGraph() == TestGraph);
	}

	SECTION("Set nullptr")
	{
		Element->PublicSetParentGraph(nullptr);
		CHECK(Element->PublicGetGraph() == nullptr);
	}
}
