// Copyright Epic Games, Inc. All Rights Reserved.
#include "TestHarness.h"

#include "Graph/Graph.h"
#include "Graph/GraphHandle.h"

TEST_CASE("Graph::Unique Index::Constructor", "[graph][uniqueidx]")
{
	SECTION("Temp Only Constructor")
	{
		FGraphUniqueIndex Test1{false};
		CHECK(Test1.IsTemporary() == false);
		CHECK(Test1.IsValid() == false);

		FGraphUniqueIndex Test2{true};
		CHECK(Test2.IsTemporary() == true);
		CHECK(Test2.IsValid() == false);
	}

	SECTION("Guid + Temp Constructor")
	{
		FGuid Guid1{1, 1, 1, 1};
		FGraphUniqueIndex Test1{Guid1, false};
		CHECK(Test1.ToString() == Guid1.ToString());
		CHECK(Test1.IsTemporary() == false);
		CHECK(Test1.IsValid() == true);

		FGuid Guid2{2, 2, 2, 2};
		FGraphUniqueIndex Test2{Guid2, true};
		CHECK(Test2.ToString() == Guid2.ToString());
		CHECK(Test2.IsTemporary() == true);
		CHECK(Test2.IsValid() == true);
	}

	SECTION("Create Unique Index")
	{
		FGraphUniqueIndex Test1 = FGraphUniqueIndex::CreateUniqueIndex(false);
		CHECK(Test1.IsTemporary() == false);
		CHECK(Test1.IsValid() == true);

		FGraphUniqueIndex Test2 = FGraphUniqueIndex::CreateUniqueIndex(true);
		CHECK(Test2.IsTemporary() == true);
		CHECK(Test2.IsValid() == true);
	}
}

TEST_CASE("Graph::Unique Index::NextUniqueIndex", "[graph][uniqueidx]")
{
	SECTION("Temporary")
	{
		FGraphUniqueIndex Test1 = FGraphUniqueIndex::CreateUniqueIndex(true);
		FGraphUniqueIndex Test2 = Test1.NextUniqueIndex();
		CHECK(Test2.IsTemporary() == true);
		CHECK(Test2.IsValid() == true);
	}

	SECTION("Non-Temporary")
	{
		FGraphUniqueIndex Test1 = FGraphUniqueIndex::CreateUniqueIndex(false);
		FGraphUniqueIndex Test2 = Test1.NextUniqueIndex();
		CHECK(Test2.IsTemporary() == false);
		CHECK(Test2.IsValid() == true);
	}
}

TEST_CASE("Graph::Unique Index::Equality", "[graph][uniqueidx]")
{
	SECTION("Equality")
	{
		FGuid Guid1{1, 1, 1, 1};
		FGraphUniqueIndex Test1{Guid1};
		FGraphUniqueIndex Test2{Guid1};
		CHECK(Test1 == Test2);
	}

	SECTION("InEquality")
	{
		FGuid Guid1{1, 1, 1, 1};
		FGuid Guid2{2, 2, 2, 2};
		FGraphUniqueIndex Test1{Guid1};
		FGraphUniqueIndex Test2{Guid2};
		CHECK(Test1 != Test2);
	}
}

TEST_CASE("Graph::Unique Index::Comparison", "[graph][uniqueidx]")
{
	SECTION("Less")
	{
		FGuid Guid1{1, 1, 1, 1};
		FGuid Guid2{2, 2, 2, 2};
		FGraphUniqueIndex Test1{Guid1};
		FGraphUniqueIndex Test2{Guid2};
		CHECK(Test1 < Test2);
	}
}

TEST_CASE("Graph::Unique Index::Copy", "[graph][uniqueidx]")
{
	FGraphUniqueIndex Test1 = FGraphUniqueIndex::CreateUniqueIndex(true);
	FGraphUniqueIndex Test2 = Test1;
	CHECK(Test1 == Test2);
}

TEST_CASE("Graph::Handle::Empty Constructor", "[graph][handle]")
{
	FGraphHandle Handle;
	CHECK(Handle.IsValid() == false);
	CHECK(Handle.HasElement() == false);
	CHECK(Handle.IsComplete() == false);
	CHECK(Handle.GetUniqueIndex() == FGraphUniqueIndex{});
	CHECK(Handle.GetGraph() == nullptr);
}

TEST_CASE("Graph::Handle::Constructor", "[graph][handle]")
{
	UGraph* Graph = NewObject<UGraph>();
	REQUIRE(Graph != nullptr);
	FGuid Guid1{1, 1, 1, 1};
	FGraphUniqueIndex Test1{Guid1};
	FGraphHandle Handle{Test1, Graph};

	CHECK(Handle.IsValid() == true);
	CHECK(Handle.HasElement() == false);
	CHECK(Handle.IsComplete() == false);
	CHECK(Handle.GetUniqueIndex() == Test1);
	CHECK(Handle.GetGraph() == Graph);
}

TEST_CASE("Graph::Handle::Equality", "[graph][handle]")
{
	UGraph* Graph = NewObject<UGraph>();
	REQUIRE(Graph != nullptr);
	FGuid Guid1{1, 1, 1, 1};
	FGuid Guid2{2, 2, 2, 2};
	FGraphUniqueIndex Test1{Guid1};
	FGraphUniqueIndex Test2{Guid2};

	FGraphHandle Handle1{Test1, Graph};
	FGraphHandle Handle2{Test2, Graph};
	FGraphHandle Handle3{Test2, Graph};
	CHECK(Handle1 != Handle2);
	CHECK(Handle1 != Handle3);
	CHECK(Handle2 == Handle3);
}

TEST_CASE("Graph::Handle::Copy", "[graph][handle]")
{
	UGraph* Graph = NewObject<UGraph>();
	REQUIRE(Graph != nullptr);
	FGuid Guid1{1, 1, 1, 1};
	FGraphUniqueIndex Test1{Guid1};
	FGraphHandle Handle1{Test1, Graph};
	FGraphHandle Handle2 = Handle1;
	CHECK(Handle1 == Handle2);
}