// Copyright Epic Games, Inc. All Rights Reserved.
#include "TestHarness.h"
#include "TestGraphBuilder.h"

TEST_CASE_METHOD(FTestGraphBuilder, "Graph::Island::Create::Single Island", "[graph][island]")
{
	PopulateVertices(2, true);
	BuildLinearEdges(2);
	FinalizeEdges();

	REQUIRE(IslandHandles.Num() == 1);

	UGraphIsland* Island = IslandHandles[0].GetIsland();
	REQUIRE(Island != nullptr);

	CHECK(Island->Handle() == IslandHandles[0]);
	CHECK(Island->IsEmpty() == false);
	CHECK(Island->GetVertices().Contains(VertexHandles[0]) == true);
	CHECK(Island->GetVertices().Contains(VertexHandles[1]) == true);
	CHECK(Island->Num() == 2);

	IslandVertexParentIslandSanityCheck(IslandHandles[0]);
}

TEST_CASE_METHOD(FTestGraphBuilder, "Graph::Island::Create::Two Islands", "[graph][island]")
{
	PopulateVertices(4, false);
	BuildLinearEdges(2);
	FinalizeEdges();
	REQUIRE(IslandHandles.Num() == 2);

	FGraphIslandHandle IslandHandle1;
	{
		UGraphVertex* Vertex = VertexHandles[0].GetVertex();
		REQUIRE(Vertex != nullptr);
		IslandHandle1 = Vertex->GetParentIsland();
	}

	UGraphIsland* Island1 = IslandHandle1.GetIsland();
	REQUIRE(Island1 != nullptr);
	{
		CHECK(Island1->Handle() == IslandHandle1);
		CHECK(Island1->IsEmpty() == false);
		CHECK(Island1->GetVertices().Contains(VertexHandles[0]) == true);
		CHECK(Island1->GetVertices().Contains(VertexHandles[1]) == true);
		CHECK(Island1->GetVertices().Contains(VertexHandles[2]) == false);
		CHECK(Island1->GetVertices().Contains(VertexHandles[3]) == false);
		CHECK(Island1->Num() == 2);
	}

	FGraphIslandHandle IslandHandle2;
	{
		UGraphVertex* Vertex = VertexHandles[2].GetVertex();
		REQUIRE(Vertex != nullptr);
		IslandHandle2 = Vertex->GetParentIsland();
	}
	CHECK(IslandHandle1 != IslandHandle2);
	UGraphIsland* Island2 = IslandHandle2.GetIsland();
	REQUIRE(Island2 != nullptr);
	{
		CHECK(Island2->Handle() == IslandHandle2);
		CHECK(Island2->IsEmpty() == false);
		CHECK(Island2->GetVertices().Contains(VertexHandles[0]) == false);
		CHECK(Island2->GetVertices().Contains(VertexHandles[1]) == false);
		CHECK(Island2->GetVertices().Contains(VertexHandles[2]) == true);
		CHECK(Island2->GetVertices().Contains(VertexHandles[3]) == true);
		CHECK(Island2->Num() == 2);
	}

	IslandVertexParentIslandSanityCheck(IslandHandle1);
	IslandVertexParentIslandSanityCheck(IslandHandle2);
}

TEST_CASE_METHOD(FTestGraphBuilder, "Graph::Island::Bulk Create Edges", "[graph][island]")
{
	PopulateVertices(10, false);
	BuildLinearEdges(2);

	{
		TArray<FEdgeSpecifier> AllEdgeParams;
		{
			FEdgeSpecifier Params{ VertexHandles[1], VertexHandles[2] };
			AllEdgeParams.Add(Params);
		}

		{
			FEdgeSpecifier Params{ VertexHandles[1], VertexHandles[3] };
			AllEdgeParams.Add(Params);
		}

		{
			FEdgeSpecifier Params{ VertexHandles[4], VertexHandles[9] };
			AllEdgeParams.Add(Params);
		}

		{
			FEdgeSpecifier Params{ VertexHandles[5], VertexHandles[6] };
			AllEdgeParams.Add(Params);
		}

		{
			FEdgeSpecifier Params{ VertexHandles[4], VertexHandles[7] };
			AllEdgeParams.Add(Params);
		}

		Graph->CreateBulkEdges(MoveTemp(AllEdgeParams));
	}

	FinalizeEdges();

	REQUIRE(IslandHandles.Num() == 2);

	FGraphIslandHandle IslandHandle1;
	{
		UGraphVertex* Vertex = VertexHandles[0].GetVertex();
		REQUIRE(Vertex != nullptr);
		IslandHandle1 = Vertex->GetParentIsland();
	}

	UGraphIsland* Island1 = IslandHandle1.GetIsland();
	REQUIRE(Island1 != nullptr);
	{
		CHECK(Island1->Handle() == IslandHandle1);
		CHECK(Island1->IsEmpty() == false);
		CHECK(Island1->GetVertices().Contains(VertexHandles[0]) == true);
		CHECK(Island1->GetVertices().Contains(VertexHandles[1]) == true);
		CHECK(Island1->GetVertices().Contains(VertexHandles[2]) == true);
		CHECK(Island1->GetVertices().Contains(VertexHandles[3]) == true);
		CHECK(Island1->Num() == 4);
	}

	FGraphIslandHandle IslandHandle2;
	{
		UGraphVertex* Vertex = VertexHandles[4].GetVertex();
		REQUIRE(Vertex != nullptr);
		IslandHandle2 = Vertex->GetParentIsland();
	}
	CHECK(IslandHandle1 != IslandHandle2);
	UGraphIsland* Island2 = IslandHandle2.GetIsland();
	REQUIRE(Island2 != nullptr);
	{
		CHECK(Island2->Handle() == IslandHandle2);
		CHECK(Island2->IsEmpty() == false);
		CHECK(Island2->GetVertices().Contains(VertexHandles[4]) == true);
		CHECK(Island2->GetVertices().Contains(VertexHandles[5]) == true);
		CHECK(Island2->GetVertices().Contains(VertexHandles[6]) == true);
		CHECK(Island2->GetVertices().Contains(VertexHandles[7]) == true);
		CHECK(Island2->GetVertices().Contains(VertexHandles[8]) == true);
		CHECK(Island2->GetVertices().Contains(VertexHandles[9]) == true);
		CHECK(Island2->Num() == 6);
	}

	IslandVertexParentIslandSanityCheck(IslandHandle1);
	IslandVertexParentIslandSanityCheck(IslandHandle2);
}

TEST_CASE_METHOD(FTestGraphBuilder, "Graph::Island::Remove Vertex::Still Connected", "[graph][island]")
{
	PopulateVertices(4, true);
	BuildFullyConnectedEdges(4);

	REQUIRE(IslandHandles.Num() == 1);

	UGraphIsland* Island1 = IslandHandles[0].GetIsland();
	REQUIRE(Island1 != nullptr);
	IslandVertexParentIslandSanityCheck(IslandHandles[0]);

	{
		CHECK(Island1->Num() == 4);
		CHECK(Island1->GetVertices().Contains(VertexHandles[0]) == true);
		CHECK(Island1->GetVertices().Contains(VertexHandles[1]) == true);
		CHECK(Island1->GetVertices().Contains(VertexHandles[2]) == true);
		CHECK(Island1->GetVertices().Contains(VertexHandles[3]) == true);
	}

	Graph->RemoveVertex(VertexHandles[0]);
	CHECK(Graph->NumIslands() == 1);
	IslandVertexParentIslandSanityCheck(IslandHandles[0]);

	{
		CHECK(Island1->Num() == 3);
		CHECK(Island1->GetVertices().Contains(VertexHandles[0]) == false);
		CHECK(Island1->GetVertices().Contains(VertexHandles[1]) == true);
		CHECK(Island1->GetVertices().Contains(VertexHandles[2]) == true);
		CHECK(Island1->GetVertices().Contains(VertexHandles[3]) == true);
	}

	Graph->RemoveVertex(VertexHandles[1]);
	CHECK(Graph->NumIslands() == 1);
	IslandVertexParentIslandSanityCheck(IslandHandles[0]);

	{
		CHECK(Island1->Num() == 2);
		CHECK(Island1->GetVertices().Contains(VertexHandles[0]) == false);
		CHECK(Island1->GetVertices().Contains(VertexHandles[1]) == false);
		CHECK(Island1->GetVertices().Contains(VertexHandles[2]) == true);
		CHECK(Island1->GetVertices().Contains(VertexHandles[3]) == true);
	}

	Graph->RemoveVertex(VertexHandles[2]);
	CHECK(Graph->NumIslands() == 1);
	IslandVertexParentIslandSanityCheck(IslandHandles[0]);

	{
		CHECK(Island1->Num() == 1);
		CHECK(Island1->GetVertices().Contains(VertexHandles[0]) == false);
		CHECK(Island1->GetVertices().Contains(VertexHandles[1]) == false);
		CHECK(Island1->GetVertices().Contains(VertexHandles[2]) == false);
		CHECK(Island1->GetVertices().Contains(VertexHandles[3]) == true);
	}

	IslandVertexParentIslandSanityCheck(IslandHandles[0]);
	CHECK(Graph->NumIslands() == 1);
}

TEST_CASE_METHOD(FTestGraphBuilder, "Graph::Island::Remove Vertex::Split", "[graph][island]")
{
	PopulateVertices(4, true);
	BuildLinearEdges(4);
	REQUIRE(IslandHandles.Num() == 1);
	CHECK(Graph->NumIslands() == 1);

	Graph->RemoveVertex(VertexHandles[1]);
	FinalizeEdges();
	REQUIRE(IslandHandles.Num() == 2);
	CHECK(Graph->NumIslands() == 2);

	FGraphIslandHandle IslandHandle1;
	{
		UGraphVertex* Vertex = VertexHandles[0].GetVertex();
		REQUIRE(Vertex != nullptr);
		IslandHandle1 = Vertex->GetParentIsland();
	}

	UGraphIsland* Island1 = IslandHandle1.GetIsland();
	REQUIRE(Island1 != nullptr);
	{
		CHECK(Island1->Handle() == IslandHandle1);
		CHECK(Island1->IsEmpty() == false);
		CHECK(Island1->GetVertices().Contains(VertexHandles[0]) == true);
		CHECK(Island1->Num() == 1);
	}

	FGraphIslandHandle IslandHandle2;
	{
		UGraphVertex* Vertex = VertexHandles[2].GetVertex();
		REQUIRE(Vertex != nullptr);
		IslandHandle2 = Vertex->GetParentIsland();
	}
	CHECK(IslandHandle1 != IslandHandle2);
	UGraphIsland* Island2 = IslandHandle2.GetIsland();
	REQUIRE(Island2 != nullptr);
	{
		CHECK(Island2->Handle() == IslandHandle2);
		CHECK(Island2->IsEmpty() == false);
		CHECK(Island2->GetVertices().Contains(VertexHandles[2]) == true);
		CHECK(Island2->GetVertices().Contains(VertexHandles[3]) == true);
		CHECK(Island2->Num() == 2);
	}

	IslandVertexParentIslandSanityCheck(IslandHandle1);
	IslandVertexParentIslandSanityCheck(IslandHandle2);
}

TEST_CASE_METHOD(FTestGraphBuilder, "Graph::Island::Operation Allowed::Flag", "[graph][island]")
{
	PopulateVertices(2, true);
	BuildLinearEdges(2);

	REQUIRE(IslandHandles.Num() == 1);

	UGraphIsland* Island = IslandHandles[0].GetIsland();
	REQUIRE(Island != nullptr);

	TArray<EGraphIslandOperations> AllFlags =
	{
		EGraphIslandOperations::Add,
		EGraphIslandOperations::Split,
		EGraphIslandOperations::Merge,
		EGraphIslandOperations::Destroy
	};

	for (EGraphIslandOperations Flag : AllFlags)
	{
		CHECK(Island->IsOperationAllowed(Flag) == true);
	}

	auto RunTest = [&AllFlags, Island](EGraphIslandOperations FlagToDisable)
	{
		Island->SetOperationAllowed(FlagToDisable, false);
		for (EGraphIslandOperations Flag : AllFlags)
		{
			if (Flag == FlagToDisable)
			{
				CHECK(Island->IsOperationAllowed(Flag) == false);
			}
			else
			{
				CHECK(Island->IsOperationAllowed(Flag) == true);
			}
		}

		Island->SetOperationAllowed(FlagToDisable, true);
		for (EGraphIslandOperations Flag : AllFlags)
		{
			CHECK(Island->IsOperationAllowed(Flag) == true);
		}
	};

	RunTest(EGraphIslandOperations::Add);
	RunTest(EGraphIslandOperations::Split);
	RunTest(EGraphIslandOperations::Merge);
	RunTest(EGraphIslandOperations::Destroy);
}

TEST_CASE_METHOD(FTestGraphBuilder, "Graph::Island::Operation Allowed::Disable Add", "[graph][island]")
{
	PopulateVertices(4, true);
	BuildLinearEdges(4);
	IslandVertexParentIslandSanityCheck(IslandHandles[0]);

	REQUIRE(IslandHandles.Num() == 1);

	UGraphIsland* Island = IslandHandles[0].GetIsland();
	REQUIRE(Island != nullptr);
	Island->SetOperationAllowed(EGraphIslandOperations::Add, false);

	FGraphVertexHandle NewVertex = Graph->CreateVertex();

	FEdgeSpecifier Params{ NewVertex, VertexHandles[0] };
	Graph->CreateBulkEdges({ Params });

	CHECK(Island->Num() == 4);
	CHECK(Island->GetVertices().Contains(NewVertex) == false);
	CHECK(NewVertex.GetVertex()->GetParentIsland() != Island->Handle());
	CHECK(NewVertex.GetVertex()->GetParentIsland() == FGraphIslandHandle::Invalid);
	CHECK(Graph->NumVertices() == 5);
	CHECK(Graph->NumIslands() == 1);
	IslandVertexParentIslandSanityCheck(IslandHandles[0]);
	CHECK(NewVertex.GetVertex()->HasEdgeTo(VertexHandles[0]) == false);
	CHECK(VertexHandles[0].GetVertex()->HasEdgeTo(NewVertex) == false);
}

TEST_CASE_METHOD(FTestGraphBuilder, "Graph::Island::Operation Allowed::Disable Split", "[graph][island]")
{
	PopulateVertices(6, true);
	BuildLinearEdges(6);
	IslandVertexParentIslandSanityCheck(IslandHandles[0]);

	REQUIRE(IslandHandles.Num() == 1);

	UGraphIsland* Island = IslandHandles[0].GetIsland();
	REQUIRE(Island != nullptr);
	Island->SetOperationAllowed(EGraphIslandOperations::Split, false);

	CHECK(Island->Num() == 6);
	CHECK(Graph->NumVertices() == 6);
	CHECK(Graph->NumIslands() == 1);

	UGraphVertex* Vertex = VertexHandles[2].GetVertex();
	REQUIRE(Vertex != nullptr);

	UGraphIsland* Island1 = nullptr;
	UGraphIsland* Island2 = nullptr;

	SECTION("Remove Vertex")
	{
		Graph->RemoveVertex(VertexHandles[2]);
		IslandVertexParentIslandSanityCheck(IslandHandles[0]);
		CHECK(Island->GetVertices().Contains(VertexHandles[0]) == true);
		CHECK(Island->GetVertices().Contains(VertexHandles[1]) == true);
		CHECK(Island->GetVertices().Contains(VertexHandles[3]) == true);
		CHECK(Island->GetVertices().Contains(VertexHandles[4]) == true);
		CHECK(Island->GetVertices().Contains(VertexHandles[5]) == true);
		CHECK(Island->Num() == 5);
		CHECK(Graph->NumVertices() == 5);

		CHECK(Graph->NumIslands() == 1);
		Island->SetOperationAllowed(EGraphIslandOperations::Split, true);
		Graph->RefreshIslandConnectivity(Island->Handle());

		Island1 = VertexHandles[0].GetVertex()->GetParentIsland().GetIsland();
		Island2 = VertexHandles[5].GetVertex()->GetParentIsland().GetIsland();

		CHECK(Island1->Num() == 2);
		CHECK(Island1->GetVertices().Contains(VertexHandles[0]) == true);
		CHECK(Island1->GetVertices().Contains(VertexHandles[1]) == true);
		CHECK(Island2->Num() == 3);
		CHECK(Island2->GetVertices().Contains(VertexHandles[3]) == true);
		CHECK(Island2->GetVertices().Contains(VertexHandles[4]) == true);
		CHECK(Island2->GetVertices().Contains(VertexHandles[5]) == true);
		CHECK(Graph->NumVertices() == 5);
	}

	CHECK(Graph->NumIslands() == 2);
	IslandVertexParentIslandSanityCheck(Island1->Handle());
	IslandVertexParentIslandSanityCheck(Island2->Handle());
}

TEST_CASE_METHOD(FTestGraphBuilder, "Graph::Island::Operation Allowed::Disable Merge::Existing Vertices", "[graph][island]")
{
	PopulateVertices(4, true);
	BuildLinearEdges(2);

	REQUIRE(IslandHandles.Num() == 2);
	IslandVertexParentIslandSanityCheck(IslandHandles[0]);
	IslandVertexParentIslandSanityCheck(IslandHandles[1]);

	auto RunTest = [this]()
	{
		FEdgeSpecifier Params{ VertexHandles[0], VertexHandles[2] };
		Graph->CreateBulkEdges({ Params });
		CHECK(Graph->NumVertices() == 4);
		CHECK(Graph->NumIslands() == 2);
		VerifyEdges({ Params }, false);

		UGraphIsland* Island0 = VertexHandles[0].GetVertex()->GetParentIsland().GetIsland();
		REQUIRE(Island0 != nullptr);
		CHECK(Island0->Num() == 2);
		CHECK(Island0->GetVertices().Contains(VertexHandles[0]) == true);
		CHECK(Island0->GetVertices().Contains(VertexHandles[1]) == true);

		UGraphIsland* Island1 = VertexHandles[2].GetVertex()->GetParentIsland().GetIsland();
		REQUIRE(Island1 != nullptr);
		CHECK(Island1->Num() == 2);
		CHECK(Island0->Handle() != Island1->Handle());
		CHECK(Island1->GetVertices().Contains(VertexHandles[2]) == true);
		CHECK(Island1->GetVertices().Contains(VertexHandles[3]) == true);

		IslandVertexParentIslandSanityCheck(Island0->Handle());
		IslandVertexParentIslandSanityCheck(Island1->Handle());
	};

	SECTION("Single disable - 1")
	{
		UGraphIsland* Island = IslandHandles[0].GetIsland();
		REQUIRE(Island != nullptr);
		Island->SetOperationAllowed(EGraphIslandOperations::Merge, false);

		RunTest();
	}

	SECTION("Single disable - 2")
	{
		UGraphIsland* Island = IslandHandles[1].GetIsland();
		REQUIRE(Island != nullptr);
		Island->SetOperationAllowed(EGraphIslandOperations::Merge, false);

		RunTest();
	}

	SECTION("Double disable")
	{
		UGraphIsland* Island0 = IslandHandles[0].GetIsland();
		REQUIRE(Island0 != nullptr);
		Island0->SetOperationAllowed(EGraphIslandOperations::Merge, false);

		UGraphIsland* Island1 = IslandHandles[1].GetIsland();
		REQUIRE(Island1 != nullptr);
		Island1->SetOperationAllowed(EGraphIslandOperations::Merge, false);

		RunTest();
	}
}

TEST_CASE_METHOD(FTestGraphBuilder, "Graph::Island::Operation Allowed::Disable Merge::New Vertex::Bulk Edges", "[graph][island]")
{
	PopulateVertices(4, true);
	BuildLinearEdges(2);

	REQUIRE(IslandHandles.Num() == 2);
	IslandVertexParentIslandSanityCheck(IslandHandles[0]);
	IslandVertexParentIslandSanityCheck(IslandHandles[1]);

	auto RunTest = [this]()
	{
		FGraphVertexHandle NewVertex = Graph->CreateVertex();
		FEdgeSpecifier Params1{VertexHandles[0], NewVertex};
		FEdgeSpecifier Params2{VertexHandles[3], NewVertex};
		Graph->CreateBulkEdges({ Params1, Params2 });
		Graph->FinalizeVertex(NewVertex);

		VerifyEdges( { Params1 }, true);
		VerifyEdges( { Params2 }, false);

		CHECK(Graph->NumVertices() == 5);
		CHECK(Graph->NumIslands() == 2);

		UGraphIsland* Island0 = VertexHandles[0].GetVertex()->GetParentIsland().GetIsland();
		REQUIRE(Island0 != nullptr);
		CHECK(Island0->GetVertices().Contains(VertexHandles[0]) == true);
		CHECK(Island0->GetVertices().Contains(VertexHandles[1]) == true);

		UGraphIsland* Island1 = VertexHandles[2].GetVertex()->GetParentIsland().GetIsland();
		REQUIRE(Island1 != nullptr);
		CHECK(Island0->Handle() != Island1->Handle());
		CHECK(Island1->GetVertices().Contains(VertexHandles[2]) == true);
		CHECK(Island1->GetVertices().Contains(VertexHandles[3]) == true);

		CHECK(Island0->Num() == 3);
		CHECK(Island0->GetVertices().Contains(NewVertex) == true);
		CHECK(Island1->Num() == 2);

		IslandVertexParentIslandSanityCheck(Island0->Handle());
		IslandVertexParentIslandSanityCheck(Island1->Handle());
	};

	SECTION("Single disable - 1")
	{
		UGraphIsland* Island = VertexHandles[0].GetVertex()->GetParentIsland().GetIsland();
		REQUIRE(Island != nullptr);
		Island->SetOperationAllowed(EGraphIslandOperations::Merge, false);

		RunTest();
	}

	SECTION("Single disable - 2")
	{
		UGraphIsland* Island = VertexHandles[3].GetVertex()->GetParentIsland().GetIsland();
		REQUIRE(Island != nullptr);
		Island->SetOperationAllowed(EGraphIslandOperations::Merge, false);

		RunTest();
	}

	SECTION("Double disable")
	{
		UGraphIsland* Island0 = VertexHandles[0].GetVertex()->GetParentIsland().GetIsland();
		REQUIRE(Island0 != nullptr);
		Island0->SetOperationAllowed(EGraphIslandOperations::Merge, false);

		UGraphIsland* Island1 = VertexHandles[3].GetVertex()->GetParentIsland().GetIsland();
		REQUIRE(Island1 != nullptr);
		Island1->SetOperationAllowed(EGraphIslandOperations::Merge, false);

		RunTest();
	}
}

TEST_CASE_METHOD(FTestGraphBuilder, "Graph::Island::Operation Allowed::Disable Destroy", "[graph][island]")
{
	PopulateVertices(4, true);
	BuildFullyConnectedEdges(4);

	REQUIRE(IslandHandles.Num() == 1);
	CHECK(Graph->NumIslands() == 1);
	IslandVertexParentIslandSanityCheck(IslandHandles[0]);
	CHECK(IslandHandles[0].IsComplete());

	UGraphIsland* Island = IslandHandles[0].GetIsland();
	REQUIRE(Island != nullptr);
	Island->SetOperationAllowed(EGraphIslandOperations::Destroy, false);

	Graph->RemoveIsland(IslandHandles[0]);

	CHECK(Graph->NumIslands() == 1);
	IslandVertexParentIslandSanityCheck(IslandHandles[0]);
	CHECK(IslandHandles[0].IsComplete());
	CHECK(Graph->GetIslands().Contains(IslandHandles[0]) == true);
	CHECK(Island->Num() == 4);
	CHECK(Island->GetVertices().Contains(VertexHandles[0]) == true);
	CHECK(Island->GetVertices().Contains(VertexHandles[1]) == true);
	CHECK(Island->GetVertices().Contains(VertexHandles[2]) == true);
	CHECK(Island->GetVertices().Contains(VertexHandles[3]) == true);
}

TEST_CASE_METHOD(FTestGraphBuilder, "Graph::Island::Events::Vertex Added", "[graph][island]")
{
	PopulateVertices(2, true);
	BuildLinearEdges(2);

	REQUIRE(IslandHandles.Num() == 1);

	UGraphIsland* Island = IslandHandles[0].GetIsland();
	REQUIRE(Island != nullptr);
	IslandVertexParentIslandSanityCheck(IslandHandles[0]);

	FGraphIslandHandle CallbackIslandHandle;
	FGraphVertexHandle CallbackVertexHandle;
	Island->OnVertexAdded.AddLambda(
		[&CallbackIslandHandle, &CallbackVertexHandle](const FGraphIslandHandle& InIslandHandle, const FGraphVertexHandle& InVertexHandle)
		{
			CallbackIslandHandle = InIslandHandle;
			CallbackVertexHandle = InVertexHandle;
		}
	);

	FGraphVertexHandle NewVertex = Graph->CreateVertex();
	FEdgeSpecifier Params{ NewVertex, VertexHandles[0] };
	Graph->CreateBulkEdges({ Params });

	CHECK(CallbackIslandHandle == Island->Handle());
	CHECK(CallbackVertexHandle == NewVertex);
}

TEST_CASE_METHOD(FTestGraphBuilder, "Graph::Island::Events::Vertex Removed", "[graph][island]")
{
	PopulateVertices(3, true);
	BuildLinearEdges(3);

	REQUIRE(IslandHandles.Num() == 1);

	UGraphIsland* Island = IslandHandles[0].GetIsland();
	REQUIRE(Island != nullptr);
	IslandVertexParentIslandSanityCheck(IslandHandles[0]);

	FGraphIslandHandle CallbackIslandHandle;
	FGraphVertexHandle CallbackVertexHandle;
	Island->OnVertexRemoved.AddLambda(
		[&CallbackIslandHandle, &CallbackVertexHandle](const FGraphIslandHandle& InIslandHandle, const FGraphVertexHandle& InVertexHandle)
		{
			CallbackIslandHandle = InIslandHandle;
			CallbackVertexHandle = InVertexHandle;
		}
	);

	SECTION("Remove Vertex")
	{
		Graph->RemoveVertex(VertexHandles[0]);
		CHECK(CallbackIslandHandle == Island->Handle());
		CHECK(CallbackVertexHandle == VertexHandles[0]);
	}
}

TEST_CASE_METHOD(FTestGraphBuilder, "Graph::Island::Events::On Destroyed", "[graph][island]")
{
	PopulateVertices(3, true);
	BuildLinearEdges(3);

	REQUIRE(IslandHandles.Num() == 1);

	UGraphIsland* Island = IslandHandles[0].GetIsland();
	REQUIRE(Island != nullptr);
	IslandVertexParentIslandSanityCheck(IslandHandles[0]);

	FGraphIslandHandle CallbackIslandHandle;
	Island->OnDestroyed.AddLambda(
		[&CallbackIslandHandle](const FGraphIslandHandle& InIslandHandle)
		{
			CallbackIslandHandle = InIslandHandle;
		}
	);

	Graph->RemoveIsland(IslandHandles[0]);
	CHECK(CallbackIslandHandle == IslandHandles[0]);
}

TEST_CASE_METHOD(FTestGraphBuilder, "Graph::Island::Events::On Connectivity Changed", "[graph][island]")
{
	PopulateVertices(3, false);

	SECTION("Finalize Vertex")
	{
		FGraphIslandHandle CreatedIslandHandle;
		FGraphIslandHandle CallbackIslandHandle;
		Graph->OnIslandCreated.AddLambda(
			[&CreatedIslandHandle, &CallbackIslandHandle](const FGraphIslandHandle& InIslandHandle)
			{
				CreatedIslandHandle = InIslandHandle;

				UGraphIsland* Island = CreatedIslandHandle.GetIsland();
				REQUIRE(Island != nullptr);

				Island->OnConnectivityChanged.AddLambda(
					[&CallbackIslandHandle](const FGraphIslandHandle& InCallbackIslandHandle, EGraphIslandConnectivityChange Change)
					{
						CallbackIslandHandle = InCallbackIslandHandle;
					}
				);
			}
		);

		Graph->FinalizeVertex(VertexHandles[0]);
		CHECK(CreatedIslandHandle.IsComplete() == true);
		CHECK(CallbackIslandHandle.IsComplete() == true);
		CHECK(CreatedIslandHandle == CallbackIslandHandle);
	}

	SECTION("Split island")
	{
		FinalizeVertices();
		BuildLinearEdges(3);

		REQUIRE(IslandHandles.Num() == 1);

		UGraphIsland* Island = IslandHandles[0].GetIsland();
		REQUIRE(Island != nullptr);
		IslandVertexParentIslandSanityCheck(IslandHandles[0]);

		FGraphIslandHandle CallbackIslandHandle;
		Island->OnConnectivityChanged.AddLambda(
			[&CallbackIslandHandle](const FGraphIslandHandle& InCallbackIslandHandle, EGraphIslandConnectivityChange Change)
			{
				CallbackIslandHandle = InCallbackIslandHandle;
			}
		);

		Graph->RemoveVertex(VertexHandles[1]);

		CHECK(CallbackIslandHandle.IsComplete());
		CHECK(CallbackIslandHandle == IslandHandles[0]);
	}
}