// Copyright Epic Games, Inc. All Rights Reserved.
#include "TestHarness.h"
#include "TestGraphBuilder.h"

#include "Graph/GraphDefaultSerialization.h"

TEST_CASE_METHOD(FTestGraphBuilder, "Graph::Serialization::Write", "[graph][serialization]")
{
	PopulateVertices(6, true);
	BuildLinearEdges(3);

	CHECK(Graph->NumVertices() == 6);
	CHECK(Graph->NumIslands() == 2);

	FDefaultGraphSerialization Serializer;
	Serializer << *Graph;
	const FSerializableGraph& SerializedGraph = Serializer.GetData();
	CHECK(SerializedGraph.Properties == Graph->GetProperties());
	CHECK(SerializedGraph.Vertices.Num() == 6);
	CHECK(SerializedGraph.Vertices.Contains(VertexHandles[0]) == true);
	CHECK(SerializedGraph.Vertices.Contains(VertexHandles[1]) == true);
	CHECK(SerializedGraph.Vertices.Contains(VertexHandles[2]) == true);
	CHECK(SerializedGraph.Vertices.Contains(VertexHandles[3]) == true);
	CHECK(SerializedGraph.Vertices.Contains(VertexHandles[4]) == true);
	CHECK(SerializedGraph.Vertices.Contains(VertexHandles[5]) == true);

	CHECK(SerializedGraph.Edges.Num() == 4);

	TSet<FEdgeSpecifier> SerializedEdges;
	TSet<FEdgeSpecifier> TestEdges;

	for (const FSerializedEdgeData& EdgeData : SerializedGraph.Edges)
	{
		const FEdgeSpecifier SerializedEdge{EdgeData.Node1, EdgeData.Node2};
		SerializedEdges.Add(SerializedEdge);
	}

	for (const FGraphVertexHandle& VertexHandle : VertexHandles)
	{
		UGraphVertex* Vertex = VertexHandle.GetVertex();
		REQUIRE(Vertex != nullptr);

		Vertex->ForEachAdjacentVertex(
			[&TestEdges, &VertexHandle](const FGraphVertexHandle& NeighborVertexHandle)
			{
				const FEdgeSpecifier TestEdge{VertexHandle, NeighborVertexHandle};
				TestEdges.Add(TestEdge);
			}
		);
	}
	CHECK(SerializedEdges.Num() == TestEdges.Num());
	for (const FEdgeSpecifier& Edge : TestEdges)
	{
		CHECK(SerializedEdges.Contains(Edge) == true);
	}

	CHECK(SerializedGraph.Islands.Num() == 2);
	CHECK(SerializedGraph.Islands.Contains(IslandHandles[0]) == true);
	CHECK(SerializedGraph.Islands.Contains(IslandHandles[1]) == true);
	for (const FGraphIslandHandle& IslandHandle : IslandHandles)
	{
		UGraphIsland* Island = IslandHandle.GetIsland();
		REQUIRE(Island != nullptr);

		REQUIRE(SerializedGraph.Islands.Contains(IslandHandle) == true);
		const FSerializedIslandData& IslandData = SerializedGraph.Islands[IslandHandle];

		CHECK(IslandData.Vertices.Num() == Island->Num());
		for (const FGraphVertexHandle& VertexHandle : Island->GetVertices())
		{
			CHECK(IslandData.Vertices.Contains(VertexHandle) == true);
		}
	}
}

TEST_CASE_METHOD(FTestGraphBuilder, "Graph::Serialization::Read (normal)", "[graph][serialization]")
{
	FSerializableGraph SerializedGraph;
	SerializedGraph.Properties.bGenerateIslands = true;

	SerializedGraph.Vertices.Add(FGraphVertexHandle { FGraphUniqueIndex::CreateUniqueIndex(), nullptr });
	SerializedGraph.Vertices.Add(FGraphVertexHandle { FGraphUniqueIndex::CreateUniqueIndex(), nullptr });
	SerializedGraph.Vertices.Add(FGraphVertexHandle { FGraphUniqueIndex::CreateUniqueIndex(), nullptr });
	SerializedGraph.Vertices.Add(FGraphVertexHandle { FGraphUniqueIndex::CreateUniqueIndex(), nullptr });
	SerializedGraph.Vertices.Add(FGraphVertexHandle { FGraphUniqueIndex::CreateUniqueIndex(), nullptr });
	SerializedGraph.Vertices.Add(FGraphVertexHandle { FGraphUniqueIndex::CreateUniqueIndex(), nullptr });

	SerializedGraph.Edges.Add(FSerializedEdgeData{ SerializedGraph.Vertices[0], SerializedGraph.Vertices[1] });
	SerializedGraph.Edges.Add(FSerializedEdgeData{ SerializedGraph.Vertices[1], SerializedGraph.Vertices[2] });
	SerializedGraph.Edges.Add(FSerializedEdgeData{ SerializedGraph.Vertices[3], SerializedGraph.Vertices[4] });
	SerializedGraph.Edges.Add(FSerializedEdgeData{ SerializedGraph.Vertices[4], SerializedGraph.Vertices[5] });

	SerializedGraph.Islands.Add(FGraphIslandHandle { FGraphUniqueIndex::CreateUniqueIndex(), nullptr }, FSerializedIslandData { TArray{SerializedGraph.Vertices[0], SerializedGraph.Vertices[1], SerializedGraph.Vertices[2]} });
	SerializedGraph.Islands.Add(FGraphIslandHandle { FGraphUniqueIndex::CreateUniqueIndex(), nullptr }, FSerializedIslandData { TArray{SerializedGraph.Vertices[3], SerializedGraph.Vertices[4], SerializedGraph.Vertices[5]} });

	FDefaultGraphDeserialization Deserializer { SerializedGraph };
	Deserializer >> *Graph;

	CHECK(SerializedGraph.Properties == Graph->GetProperties());
	CHECK(Graph->NumVertices() == 6);
	CHECK(Graph->NumIslands() == 2);

	int32 Index = 0;
	for (const FGraphVertexHandle& SerializedVertexHandle : SerializedGraph.Vertices)
	{
		CHECK(Graph->GetVertices().Contains(SerializedVertexHandle) == true);

		UGraphVertex* LoadedVertex = Graph->GetVertices().FindRef(SerializedVertexHandle);
		REQUIRE(LoadedVertex != nullptr);
		CHECK(SerializedVertexHandle == LoadedVertex->Handle());
	}

	for (const FSerializedEdgeData& Edge: SerializedGraph.Edges)
	{
		CHECK(Graph->GetVertices().Contains(Edge.Node1) == true);
		CHECK(Graph->GetVertices().Contains(Edge.Node2) == true);

		UGraphVertex* Vertex1 = Graph->GetVertices().FindRef(Edge.Node1);
		REQUIRE(Vertex1 != nullptr);

		UGraphVertex* Vertex2 = Graph->GetVertices().FindRef(Edge.Node2);
		REQUIRE(Vertex2 != nullptr);

		CHECK(Vertex1->HasEdgeTo(Edge.Node2) == true);
		CHECK(Vertex2->HasEdgeTo(Edge.Node1) == true);
	}

	for (const TPair<FGraphIslandHandle, FSerializedIslandData>& Island : SerializedGraph.Islands)
	{
		CHECK(Graph->GetIslands().Contains(Island.Key) == true);

		UGraphIsland* LoadedIsland = Graph->GetIslands().FindRef(Island.Key);
		REQUIRE(LoadedIsland != nullptr);

		for (const FGraphVertexHandle& IslandVertexHandle : Island.Value.Vertices)
		{
			CHECK(LoadedIsland->GetVertices().Contains(IslandVertexHandle) == true);

			const FGraphVertexHandle* LoadedIslandVertexHandle = LoadedIsland->GetVertices().Find(IslandVertexHandle);
			REQUIRE(LoadedIslandVertexHandle != nullptr);
			CHECK(LoadedIslandVertexHandle->IsComplete() == true);
			CHECK(LoadedIslandVertexHandle->GetGraph() == Graph);
		}
	}
}

TEST_CASE_METHOD(FTestGraphBuilder, "Graph::Serialization::Read::Errors::Vertex::Invalid Index", "[graph][serialization]")
{
	FSerializableGraph SerializedGraph;
	SerializedGraph.Properties.bGenerateIslands = true;
	SerializedGraph.Vertices.Add(FGraphVertexHandle { FGraphUniqueIndex{ FGuid{0, 0, 0, 0} }, nullptr });

	FDefaultGraphDeserialization Deserializer { SerializedGraph };
	Deserializer >> *Graph;
	CHECK(SerializedGraph.Properties == Graph->GetProperties());
	CHECK(Graph->NumVertices() == 0);
	CHECK(Graph->NumIslands() == 0);
}

TEST_CASE_METHOD(FTestGraphBuilder, "Graph::Serialization::Read::Errors::Edge::Invalid Vertex Index", "[graph][serialization]")
{
	FSerializableGraph SerializedGraph;
	SerializedGraph.Properties.bGenerateIslands = true;
	SerializedGraph.Vertices.Add(FGraphVertexHandle { FGraphUniqueIndex{FGuid{0, 0, 0, 1}}, nullptr });
	SerializedGraph.Vertices.Add(FGraphVertexHandle { FGraphUniqueIndex{FGuid{0, 0, 0, 2}}, nullptr });
	SerializedGraph.Edges.Add(FSerializedEdgeData{ SerializedGraph.Vertices[0], FGraphVertexHandle { FGraphUniqueIndex{FGuid{0, 0, 0, 3}}, nullptr } });

	FDefaultGraphDeserialization Deserializer { SerializedGraph };
	Deserializer >> *Graph;
	CHECK(SerializedGraph.Properties == Graph->GetProperties());
	CHECK(Graph->NumVertices() == 2);
	CHECK(Graph->NumIslands() == 0);
}

TEST_CASE_METHOD(FTestGraphBuilder, "Graph::Serialization::Read::Errors::Island::Invalid Index", "[graph][serialization]")
{
	FSerializableGraph SerializedGraph;
	SerializedGraph.Properties.bGenerateIslands = true;
	SerializedGraph.Vertices.Add(FGraphVertexHandle { FGraphUniqueIndex::CreateUniqueIndex(), nullptr });
	SerializedGraph.Vertices.Add(FGraphVertexHandle { FGraphUniqueIndex::CreateUniqueIndex(), nullptr });
	SerializedGraph.Edges.Add(FSerializedEdgeData{ SerializedGraph.Vertices[0], SerializedGraph.Vertices[1] });
	SerializedGraph.Islands.Add(FGraphIslandHandle { FGraphUniqueIndex{FGuid{0, 0, 0, 0}}, nullptr }, FSerializedIslandData { TArray{SerializedGraph.Vertices[0], SerializedGraph.Vertices[1] } });

	FDefaultGraphDeserialization Deserializer { SerializedGraph };
	Deserializer >> *Graph;
	CHECK(SerializedGraph.Properties == Graph->GetProperties());
	CHECK(Graph->NumVertices() == 2);
	CHECK(Graph->NumIslands() == 1);
	CHECK(Graph->GetIslands().Contains(FGraphIslandHandle { FGraphUniqueIndex { FGuid { 0, 0, 0, 0 } }, nullptr }) == false);
}

TEST_CASE_METHOD(FTestGraphBuilder, "Graph::Serialization::Read::Errors::Island::Invalid Vertex Index", "[graph][serialization]")
{
	FSerializableGraph SerializedGraph;
	SerializedGraph.Properties.bGenerateIslands = true;
	SerializedGraph.Vertices.Add(FGraphVertexHandle { FGraphUniqueIndex{FGuid{0, 0, 0, 1}}, nullptr });
	SerializedGraph.Vertices.Add(FGraphVertexHandle { FGraphUniqueIndex{FGuid{0, 0, 0, 2}}, nullptr });
	SerializedGraph.Edges.Add(FSerializedEdgeData{ SerializedGraph.Vertices[0], SerializedGraph.Vertices[1] });
	SerializedGraph.Islands.Add(FGraphIslandHandle { FGraphUniqueIndex{FGuid{1, 0, 0, 0}}, nullptr }, FSerializedIslandData { TArray{ SerializedGraph.Vertices[0], SerializedGraph.Vertices[1], FGraphVertexHandle { FGraphUniqueIndex{FGuid{0, 0, 0, 3}}, nullptr } } });

	FDefaultGraphDeserialization Deserializer { SerializedGraph };
	Deserializer >> *Graph;
	CHECK(SerializedGraph.Properties == Graph->GetProperties());
	CHECK(Graph->NumVertices() == 2);
	CHECK(Graph->NumIslands() == 1);

	UGraphIsland* LoadedIsland = Graph->GetIslands().FindRef(FGraphIslandHandle {FGraphUniqueIndex{FGuid{1, 0, 0, 0}}, nullptr });
	REQUIRE(LoadedIsland != nullptr);
	CHECK(LoadedIsland->Num() == 2);
	CHECK(LoadedIsland->GetVertices().Contains(SerializedGraph.Vertices[0]) == true);
	CHECK(LoadedIsland->GetVertices().Contains(SerializedGraph.Vertices[1]) == true);
	CHECK(LoadedIsland->GetVertices().Contains(FGraphVertexHandle { FGraphUniqueIndex { FGuid { 0, 0, 0, 3 } }, nullptr }) == false);
}

TEST_CASE_METHOD(FTestGraphBuilder, "Graph::Serialization::Read::Errors::Island::Edge Island Mismatch::Merge", "[graph][serialization]")
{
	FSerializableGraph SerializedGraph;
	SerializedGraph.Properties.bGenerateIslands = true;
	SerializedGraph.Vertices.Add(FGraphVertexHandle { FGraphUniqueIndex::CreateUniqueIndex(), nullptr });
	SerializedGraph.Vertices.Add(FGraphVertexHandle { FGraphUniqueIndex::CreateUniqueIndex(), nullptr });
	SerializedGraph.Vertices.Add(FGraphVertexHandle { FGraphUniqueIndex::CreateUniqueIndex(), nullptr });

	SerializedGraph.Edges.Add(FSerializedEdgeData{ SerializedGraph.Vertices[0], SerializedGraph.Vertices[1] });
	SerializedGraph.Edges.Add(FSerializedEdgeData{ SerializedGraph.Vertices[1], SerializedGraph.Vertices[2] });

	SerializedGraph.Islands.Add(FGraphIslandHandle { FGraphUniqueIndex{FGuid{1, 0, 0, 0}}, nullptr }, FSerializedIslandData { TArray { SerializedGraph.Vertices[0], SerializedGraph.Vertices[1] } });
	SerializedGraph.Islands.Add(FGraphIslandHandle { FGraphUniqueIndex{FGuid{2, 0, 0, 0}}, nullptr }, FSerializedIslandData { TArray { SerializedGraph.Vertices[2] } });

	FDefaultGraphDeserialization Deserializer { SerializedGraph };
	Deserializer >> *Graph;
	CHECK(SerializedGraph.Properties == Graph->GetProperties());
	CHECK(Graph->NumVertices() == 3);
	CHECK(Graph->NumIslands() == 1);

	CHECK(Graph->GetIslands().Contains(FGraphIslandHandle { FGraphUniqueIndex { FGuid { 1, 0, 0, 0 } }, nullptr }) == true);
	CHECK(Graph->GetIslands().Contains(FGraphIslandHandle { FGraphUniqueIndex { FGuid { 2, 0, 0, 0 } }, nullptr }) == false);

	UGraphIsland* LoadedIsland = Graph->GetIslands().FindRef(FGraphIslandHandle {FGraphUniqueIndex{FGuid{1, 0, 0, 0}}, nullptr });
	REQUIRE(LoadedIsland != nullptr);
	CHECK(LoadedIsland->Num() == 3);
	CHECK(LoadedIsland->GetVertices().Contains(SerializedGraph.Vertices[0]) == true);
	CHECK(LoadedIsland->GetVertices().Contains(SerializedGraph.Vertices[1]) == true);
	CHECK(LoadedIsland->GetVertices().Contains(SerializedGraph.Vertices[2]) == true);
}

TEST_CASE_METHOD(FTestGraphBuilder, "Graph::Serialization::Read::Errors::Island::Edge Island Mismatch::Split", "[graph][serialization]")
{
	FSerializableGraph SerializedGraph;
	SerializedGraph.Properties.bGenerateIslands = true;
	SerializedGraph.Vertices.Add(FGraphVertexHandle { FGraphUniqueIndex::CreateUniqueIndex(), nullptr });
	SerializedGraph.Vertices.Add(FGraphVertexHandle { FGraphUniqueIndex::CreateUniqueIndex(), nullptr });
	SerializedGraph.Vertices.Add(FGraphVertexHandle { FGraphUniqueIndex::CreateUniqueIndex(), nullptr });

	SerializedGraph.Edges.Add(FSerializedEdgeData{ SerializedGraph.Vertices[0], SerializedGraph.Vertices[1] });

	SerializedGraph.Islands.Add(FGraphIslandHandle { FGraphUniqueIndex{FGuid{1, 0, 0, 0}}, nullptr }, FSerializedIslandData { TArray { SerializedGraph.Vertices[0], SerializedGraph.Vertices[1], SerializedGraph.Vertices[2] } });

	FDefaultGraphDeserialization Deserializer { SerializedGraph };
	Deserializer >> *Graph;
	CHECK(SerializedGraph.Properties == Graph->GetProperties());
	CHECK(Graph->NumVertices() == 3);
	CHECK(Graph->NumIslands() == 2);

	CHECK(Graph->GetIslands().Contains(FGraphIslandHandle { FGraphUniqueIndex { FGuid { 1, 0, 0, 0 } }, nullptr }) == true);

	UGraphIsland* LoadedIsland = Graph->GetIslands().FindRef(FGraphIslandHandle {FGraphUniqueIndex{FGuid{1, 0, 0, 0}}, nullptr });
	REQUIRE(LoadedIsland != nullptr);
	CHECK(LoadedIsland->Num() == 2);
	CHECK(LoadedIsland->GetVertices().Contains(SerializedGraph.Vertices[0]) == true);
	CHECK(LoadedIsland->GetVertices().Contains(SerializedGraph.Vertices[1]) == true);
	CHECK(LoadedIsland->GetVertices().Contains(SerializedGraph.Vertices[2]) == false);

	UGraphIsland* SplitIsland = Graph->GetCompleteNodeHandle(SerializedGraph.Vertices[2]).GetVertex()->GetParentIsland().GetIsland();
	REQUIRE(SplitIsland != nullptr);
	CHECK(SplitIsland->Num() == 1);
	CHECK(SplitIsland->GetVertices().Contains(SerializedGraph.Vertices[0]) == false);
	CHECK(SplitIsland->GetVertices().Contains(SerializedGraph.Vertices[1]) == false);
	CHECK(SplitIsland->GetVertices().Contains(SerializedGraph.Vertices[2]) == true);
}

TEST_CASE_METHOD(FTestGraphBuilder, "Graph::Serialization::Read::Errors::Island::Duplicate Vertex Assignment::No Edges", "[graph][serialization]")
{
	FSerializableGraph SerializedGraph;
	SerializedGraph.Properties.bGenerateIslands = true;
	SerializedGraph.Vertices.Add(FGraphVertexHandle { FGraphUniqueIndex::CreateUniqueIndex(), nullptr });
	SerializedGraph.Vertices.Add(FGraphVertexHandle { FGraphUniqueIndex::CreateUniqueIndex(), nullptr });
	SerializedGraph.Vertices.Add(FGraphVertexHandle { FGraphUniqueIndex::CreateUniqueIndex(), nullptr });

	SerializedGraph.Islands.Add(FGraphIslandHandle { FGraphUniqueIndex{FGuid{1, 0, 0, 0}}, nullptr }, FSerializedIslandData { TArray { SerializedGraph.Vertices[0], SerializedGraph.Vertices[2] } });
	SerializedGraph.Islands.Add(FGraphIslandHandle { FGraphUniqueIndex{FGuid{2, 0, 0, 0}}, nullptr }, FSerializedIslandData { TArray { SerializedGraph.Vertices[1] } });
	SerializedGraph.Islands.Add(FGraphIslandHandle { FGraphUniqueIndex{FGuid{3, 0, 0, 0}}, nullptr }, FSerializedIslandData { TArray { SerializedGraph.Vertices[0], SerializedGraph.Vertices[1] } });

	FDefaultGraphDeserialization Deserializer { SerializedGraph };
	Deserializer >> *Graph;
	CHECK(SerializedGraph.Properties == Graph->GetProperties());
	CHECK(Graph->NumVertices() == 3);
	CHECK(Graph->NumIslands() == 3);

	CHECK(Graph->GetIslands().Contains(FGraphIslandHandle { FGraphUniqueIndex { FGuid { 1, 0, 0, 0 } }, nullptr }) == true);
	CHECK(Graph->GetIslands().Contains(FGraphIslandHandle { FGraphUniqueIndex { FGuid { 2, 0, 0, 0 } }, nullptr }) == false);
	CHECK(Graph->GetIslands().Contains(FGraphIslandHandle { FGraphUniqueIndex { FGuid { 3, 0, 0, 0 } }, nullptr }) == false);

	UGraphIsland* Island1 = Graph->GetCompleteNodeHandle(SerializedGraph.Vertices[0]).GetVertex()->GetParentIsland().GetIsland();
	REQUIRE(Island1 != nullptr);
	CHECK(Island1->Num() == 1);
	CHECK(Island1->GetVertices().Contains(SerializedGraph.Vertices[0]) == true);
	CHECK(Island1->GetVertices().Contains(SerializedGraph.Vertices[1]) == false);
	CHECK(Island1->GetVertices().Contains(SerializedGraph.Vertices[2]) == false);

	UGraphIsland* Island2 = Graph->GetCompleteNodeHandle(SerializedGraph.Vertices[1]).GetVertex()->GetParentIsland().GetIsland();
	REQUIRE(Island2 != nullptr);
	CHECK(Island2->Num() == 1);
	CHECK(Island2->GetVertices().Contains(SerializedGraph.Vertices[1]) == true);
	CHECK(Island2->GetVertices().Contains(SerializedGraph.Vertices[0]) == false);
	CHECK(Island2->GetVertices().Contains(SerializedGraph.Vertices[2]) == false);

	UGraphIsland* Island3 = Graph->GetCompleteNodeHandle(SerializedGraph.Vertices[2]).GetVertex()->GetParentIsland().GetIsland();
	REQUIRE(Island3 != nullptr);
	CHECK(Island3->Num() == 1);
	CHECK(Island3->GetVertices().Contains(SerializedGraph.Vertices[2]) == true);
	CHECK(Island3->GetVertices().Contains(SerializedGraph.Vertices[0]) == false);
	CHECK(Island3->GetVertices().Contains(SerializedGraph.Vertices[1]) == false);
}

TEST_CASE_METHOD(FTestGraphBuilder, "Graph::Serialization::Read::Errors::Island::Duplicate Vertex Assignment::With Edge", "[graph][serialization]")
{
	FSerializableGraph SerializedGraph;
	SerializedGraph.Properties.bGenerateIslands = true;
	SerializedGraph.Vertices.Add(FGraphVertexHandle { FGraphUniqueIndex::CreateUniqueIndex(), nullptr });
	SerializedGraph.Vertices.Add(FGraphVertexHandle { FGraphUniqueIndex::CreateUniqueIndex(), nullptr });

	SerializedGraph.Edges.Add(FSerializedEdgeData{ SerializedGraph.Vertices[0], SerializedGraph.Vertices[1] });

	SerializedGraph.Islands.Add(FGraphIslandHandle { FGraphUniqueIndex{FGuid{1, 0, 0, 0}}, nullptr }, FSerializedIslandData { TArray { SerializedGraph.Vertices[0] } });
	SerializedGraph.Islands.Add(FGraphIslandHandle { FGraphUniqueIndex{FGuid{2, 0, 0, 0}}, nullptr }, FSerializedIslandData { TArray { SerializedGraph.Vertices[1] } });
	SerializedGraph.Islands.Add(FGraphIslandHandle { FGraphUniqueIndex{FGuid{3, 0, 0, 0}}, nullptr }, FSerializedIslandData { TArray { SerializedGraph.Vertices[0], SerializedGraph.Vertices[1] } });

	FDefaultGraphDeserialization Deserializer { SerializedGraph };
	Deserializer >> *Graph;
	CHECK(SerializedGraph.Properties == Graph->GetProperties());
	CHECK(Graph->NumVertices() == 2);
	CHECK(Graph->NumIslands() == 1);

	CHECK(Graph->GetIslands().Contains(FGraphIslandHandle { FGraphUniqueIndex { FGuid { 1, 0, 0, 0 } }, nullptr }) == true);
	CHECK(Graph->GetIslands().Contains(FGraphIslandHandle { FGraphUniqueIndex { FGuid { 2, 0, 0, 0 } }, nullptr }) == false);
	CHECK(Graph->GetIslands().Contains(FGraphIslandHandle { FGraphUniqueIndex { FGuid { 3, 0, 0, 0 } }, nullptr }) == false);

	UGraphIsland* Island1 = Graph->GetCompleteNodeHandle(SerializedGraph.Vertices[0]).GetVertex()->GetParentIsland().GetIsland();
	REQUIRE(Island1 != nullptr);
	CHECK(Island1->Handle() == FGraphIslandHandle { FGraphUniqueIndex { FGuid { 1, 0, 0, 0 } }, nullptr });
	CHECK(Island1->Num() == 2);
	CHECK(Island1->GetVertices().Contains(SerializedGraph.Vertices[0]) == true);
	CHECK(Island1->GetVertices().Contains(SerializedGraph.Vertices[1]) == true);
}

TEST_CASE_METHOD(FTestGraphBuilder, "Graph::Serialization::Read::Errors::Island::Zero Size", "[graph][serialization]")
{
	FSerializableGraph SerializedGraph;
	SerializedGraph.Properties.bGenerateIslands = true;
	SerializedGraph.Vertices.Add(FGraphVertexHandle { FGraphUniqueIndex::CreateUniqueIndex(), nullptr });
	SerializedGraph.Vertices.Add(FGraphVertexHandle { FGraphUniqueIndex::CreateUniqueIndex(), nullptr });
	SerializedGraph.Edges.Add(FSerializedEdgeData{ SerializedGraph.Vertices[0], SerializedGraph.Vertices[1] });
	SerializedGraph.Islands.Add(FGraphIslandHandle { FGraphUniqueIndex{FGuid{1, 0, 0, 0}}, nullptr }, FSerializedIslandData { TArray { SerializedGraph.Vertices[0], SerializedGraph.Vertices[1] } });
	SerializedGraph.Islands.Add(FGraphIslandHandle { FGraphUniqueIndex{FGuid{2, 0, 0, 0}}, nullptr }, FSerializedIslandData { TArray<FGraphVertexHandle>{} });

	FDefaultGraphDeserialization Deserializer { SerializedGraph };
	Deserializer >> *Graph;
	CHECK(SerializedGraph.Properties == Graph->GetProperties());
	CHECK(Graph->NumVertices() == 2);
	CHECK(Graph->NumIslands() == 1);

	CHECK(Graph->GetIslands().Contains(FGraphIslandHandle { FGraphUniqueIndex { FGuid { 1, 0, 0, 0 } }, nullptr }) == true);
	CHECK(Graph->GetIslands().Contains(FGraphIslandHandle { FGraphUniqueIndex { FGuid { 2, 0, 0, 0 } }, nullptr }) == false);

	UGraphIsland* LoadedIsland = Graph->GetIslands().FindRef(FGraphIslandHandle {FGraphUniqueIndex{FGuid{1, 0, 0, 0}}, nullptr });
	REQUIRE(LoadedIsland != nullptr);
	CHECK(LoadedIsland->Num() == 2);
	CHECK(LoadedIsland->GetVertices().Contains(SerializedGraph.Vertices[0]) == true);
	CHECK(LoadedIsland->GetVertices().Contains(SerializedGraph.Vertices[1]) == true);
}

TEST_CASE_METHOD(FTestGraphBuilder, "Graph::Serialization::Incremental::Initialization", "[graph][serialization][incremental]")
{
	PopulateVertices(6, true);
	BuildLinearEdges(3);

	CHECK(Graph->NumVertices() == 6);
	CHECK(Graph->NumIslands() == 2);

	FDefaultGraphSerialization Serializer;
	Serializer << *Graph;

	FDefaultGraphIncrementalSerialization IncrementalSerializer(Graph);
	CHECK(Serializer.GetData() == IncrementalSerializer.GetLatestData());
}

TEST_CASE_METHOD(FTestGraphBuilder, "Graph::Serialization::Incremental::GraphBuild", "[graph][serialization][incremental]")
{
	SECTION("Linear")
	{
		FDefaultGraphIncrementalSerialization IncrementalSerializer(Graph);
		PopulateVertices(6, true);
		BuildLinearEdges(3);

		CHECK(Graph->NumVertices() == 6);
		CHECK(Graph->NumIslands() == 2);

		FDefaultGraphSerialization Serializer;
		Serializer << *Graph;
		CHECK(Serializer.GetData() == IncrementalSerializer.GetLatestData());
	}

	SECTION("Fully Connected")
	{
		FDefaultGraphIncrementalSerialization IncrementalSerializer(Graph);
		PopulateVertices(10, true);
		BuildFullyConnectedEdges(5);

		CHECK(Graph->NumVertices() == 10);
		CHECK(Graph->NumIslands() == 2);

		FDefaultGraphSerialization Serializer;
		Serializer << *Graph;
		CHECK(Serializer.GetData() == IncrementalSerializer.GetLatestData());
	}
}

TEST_CASE_METHOD(FTestGraphBuilder, "Graph::Serialization::Incremental::Removal", "[graph][serialization][incremental]")
{
	PopulateVertices(10, true);
	BuildFullyConnectedEdges(5);
	FDefaultGraphIncrementalSerialization IncrementalSerializer(Graph);

	SECTION("Remove Vertices")
	{
		Graph->RemoveVertex(VertexHandles[0]);
		Graph->RemoveVertex(VertexHandles[1]);
		Graph->RemoveVertex(VertexHandles[6]);

		{
			FDefaultGraphSerialization Serializer;
			Serializer << *Graph;
			CHECK(Serializer.GetData() == IncrementalSerializer.GetLatestData());
		}

		TArray<FGraphVertexHandle> Bulk{ VertexHandles[2], VertexHandles[7], VertexHandles[8] };
		Graph->RemoveBulkVertices(Bulk);

		{
			FDefaultGraphSerialization Serializer;
			Serializer << *Graph;
			CHECK(Serializer.GetData() == IncrementalSerializer.GetLatestData());
		}
	}

	SECTION("Remove Island")
	{
		Graph->RemoveIsland(IslandHandles[0]);
		{
			FDefaultGraphSerialization Serializer;
			Serializer << *Graph;
			CHECK(Serializer.GetData() == IncrementalSerializer.GetLatestData());
		}

		Graph->RemoveIsland(IslandHandles[0]);
		{
			FDefaultGraphSerialization Serializer;
			Serializer << *Graph;
			CHECK(Serializer.GetData() == IncrementalSerializer.GetLatestData());
		}
	}
}