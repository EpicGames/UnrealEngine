// Copyright Epic Games, Inc. All Rights Reserved.
#include "TestHarness.h"
#include "TestGraphBuilder.h"

TEST_CASE("Graph::Properties::Equality", "[graph][properties]")
{
	SECTION("Equality")
	{
		FGraphProperties Props1;
		Props1.bGenerateIslands = true;

		FGraphProperties Props2;
		Props2.bGenerateIslands = true;

		CHECK(Props1 == Props2);
	}

	SECTION("InEquality")
	{
		FGraphProperties Props1;
		Props1.bGenerateIslands = true;

		FGraphProperties Props2;
		Props2.bGenerateIslands = false;

		CHECK(Props1 != Props2);
	}
}

TEST_CASE("Graph::Default Constructor", "[graph]")
{
	TObjectPtr<UGraph> Graph = NewObject<UGraph>();

	SECTION("State")
	{
		CHECK(Graph->NumVertices() == 0);
		CHECK(Graph->NumIslands() == 0);
	}
}

TEST_CASE("Graph::Initialize Properties", "[graph]")
{
	TObjectPtr<UGraph> Graph = NewObject<UGraph>();

	SECTION("True Islands")
	{
		FGraphProperties Props1;
		Props1.bGenerateIslands = true;
		Graph->InitializeFromProperties(Props1);
		CHECK(Graph->GetProperties() == Props1);
	}

	SECTION("False Islands")
	{
		FGraphProperties Props1;
		Props1.bGenerateIslands = false;
		Graph->InitializeFromProperties(Props1);
		CHECK(Graph->GetProperties() == Props1);
	}
}

TEST_CASE_METHOD(FTestGraphBuilder, "Graph::Create Vertices", "[graph]")
{
	FGraphUniqueIndex Node1Index = { FGuid { 1, 1, 1, 1 } };
	FGraphVertexHandle Node1 = Graph->CreateVertex(Node1Index);
	{
		CHECK(Node1.GetUniqueIndex() == Node1Index);
		CHECK(Graph->NumVertices() == 1);
		CHECK(Graph->NumIslands() == 0);
		CHECK(Graph->GetVertices().Contains(Node1) == true);
	}

	FGraphUniqueIndex Node2Index = { FGuid { 2, 2, 2, 2 } };
	FGraphVertexHandle Node2 = Graph->CreateVertex(Node2Index);
	{
		CHECK(Node2.GetUniqueIndex() == Node2Index);
		CHECK(Graph->NumVertices() == 2);
		CHECK(Graph->NumIslands() == 0);
		CHECK(Graph->GetVertices().Contains(Node2) == true);
	}

	{
		{
			FGraphVertexHandle Complete1 = Graph->GetCompleteNodeHandle(FGraphVertexHandle{ Node1Index, Graph });
			CHECK(Complete1 == Node1);
			CHECK(Complete1.IsValid() == true);
			CHECK(Complete1.HasElement() == true);
			CHECK(Complete1.IsComplete() == true);
		}

		{
			FGraphVertexHandle Complete2 = Graph->GetCompleteNodeHandle(FGraphVertexHandle{ Node2Index, Graph });
			CHECK(Complete2 == Node2);
			CHECK(Complete2.IsValid() == true);
			CHECK(Complete2.HasElement() == true);
			CHECK(Complete2.IsComplete() == true);
		}

		{
			FGraphVertexHandle Complete1 = Graph->GetCompleteNodeHandle(Node1);
			CHECK(Complete1 == Node1);
			CHECK(Complete1.IsValid() == true);
			CHECK(Complete1.HasElement() == true);
			CHECK(Complete1.IsComplete() == true);
		}

		{
			FGraphVertexHandle Incomplete = Graph->GetCompleteNodeHandle(FGraphVertexHandle{ FGraphUniqueIndex{ FGuid{ 3, 3, 3, 3 } }, Graph });
			CHECK(Incomplete.IsValid() == false);
			CHECK(Incomplete.IsComplete() == false);
			CHECK(Incomplete.HasElement() == false);
		}

		{
			FGraphVertexHandle Incomplete = Graph->GetCompleteNodeHandle(FGraphVertexHandle{ FGraphUniqueIndex{}, Graph });
			CHECK(Incomplete.IsValid() == false);
			CHECK(Incomplete.IsComplete() == false);
			CHECK(Incomplete.HasElement() == false);
		}
	}

	{
		Graph->FinalizeVertex(Node1);
		CHECK(Graph->NumVertices() == 2);
		CHECK(Graph->NumIslands() == 1);
	}
}

TEST_CASE_METHOD(FTestGraphBuilder, "Graph::Create Vertices::Duplicate", "[graph]")
{
	FGraphUniqueIndex Node1Index = { FGuid { 1, 1, 1, 1 } };
	FGraphVertexHandle Node1 = Graph->CreateVertex(Node1Index);
	{
		CHECK(Node1.GetUniqueIndex() == Node1Index);
		CHECK(Graph->NumVertices() == 1);
		CHECK(Graph->NumIslands() == 0);
		CHECK(Graph->GetVertices().Contains(Node1) == true);
	}

	FGraphVertexHandle Node2 = Graph->CreateVertex(Node1Index);
	{
		CHECK(Node2.IsValid() == false);
		CHECK(Node2.IsComplete() == false);
		CHECK(Graph->NumVertices() == 1);
		CHECK(Graph->NumIslands() == 0);
		CHECK(Graph->GetVertices().Contains(Node2) == false);
	}
}

TEST_CASE_METHOD(FTestGraphBuilder, "Graph::Create Edges::Single", "[graph]")
{
	PopulateVertices(4, false);

	{
		FEdgeSpecifier Params{ VertexHandles[1], VertexHandles[0] };
		Graph->CreateBulkEdges({ Params });
		CHECK(Graph->NumIslands() == 1);
		VerifyEdges({ Params }, true);
		VertexShouldOnlyHaveEdgesTo(VertexHandles[0], {VertexHandles[1]});
		VertexShouldOnlyHaveEdgesTo(VertexHandles[1], {VertexHandles[0]});
	}

	{
		FinalizeVertices();
		CHECK(Graph->NumIslands() == 3);
	}

	{
		FEdgeSpecifier Params{ VertexHandles[2], VertexHandles[3] };
		Graph->CreateBulkEdges({ Params });
		CHECK(Graph->NumIslands() == 2);
		VerifyEdges({ Params }, true);

		VertexShouldOnlyHaveEdgesTo(VertexHandles[2], {VertexHandles[3]});
		VertexShouldOnlyHaveEdgesTo(VertexHandles[3], {VertexHandles[2]});
	}

	{
		FEdgeSpecifier Params{ VertexHandles[0], VertexHandles[2] };
		Graph->CreateBulkEdges({ Params });
		CHECK(Graph->NumIslands() == 1);
		VerifyEdges( { Params }, true);

		VertexShouldOnlyHaveEdgesTo(VertexHandles[0], {VertexHandles[1], VertexHandles[2]});
		VertexShouldOnlyHaveEdgesTo(VertexHandles[1], {VertexHandles[0]});
		VertexShouldOnlyHaveEdgesTo(VertexHandles[2], {VertexHandles[0], VertexHandles[3]});
		VertexShouldOnlyHaveEdgesTo(VertexHandles[3], {VertexHandles[2]});
	}

	{
		FEdgeSpecifier Params{ VertexHandles[0], VertexHandles[2] };
		Graph->CreateBulkEdges({ Params });
		CHECK(Graph->NumIslands() == 1);
		VerifyEdges({ Params }, true);

		VertexShouldOnlyHaveEdgesTo(VertexHandles[0], {VertexHandles[1], VertexHandles[2]});
		VertexShouldOnlyHaveEdgesTo(VertexHandles[1], {VertexHandles[0]});
		VertexShouldOnlyHaveEdgesTo(VertexHandles[2], {VertexHandles[0], VertexHandles[3]});
		VertexShouldOnlyHaveEdgesTo(VertexHandles[3], {VertexHandles[2]});
	}

	{
		FEdgeSpecifier Params{ VertexHandles[2], VertexHandles[0] };
		Graph->CreateBulkEdges({ Params });
		CHECK(Graph->NumIslands() == 1);
		VerifyEdges({ Params }, true);

		VertexShouldOnlyHaveEdgesTo(VertexHandles[0], {VertexHandles[1], VertexHandles[2]});
		VertexShouldOnlyHaveEdgesTo(VertexHandles[1], {VertexHandles[0]});
		VertexShouldOnlyHaveEdgesTo(VertexHandles[2], {VertexHandles[0], VertexHandles[3]});
		VertexShouldOnlyHaveEdgesTo(VertexHandles[3], {VertexHandles[2]});
	}
}

TEST_CASE_METHOD(FTestGraphBuilder, "Graph::Create Edges::Bulk", "[graph]")
{
	PopulateVertices(4, true);

	{
		FEdgeSpecifier Params1{ VertexHandles[1], VertexHandles[0] };
		FEdgeSpecifier Params2{ VertexHandles[2], VertexHandles[3] };
		FEdgeSpecifier Params3{ VertexHandles[0], VertexHandles[2] };
		Graph->CreateBulkEdges({ Params1, Params2, Params3 });
		CHECK(Graph->NumIslands() == 1);

		VerifyEdges({ Params1, Params2, Params3 }, true);
		VertexShouldOnlyHaveEdgesTo(VertexHandles[0], {VertexHandles[1], VertexHandles[2]});
		VertexShouldOnlyHaveEdgesTo(VertexHandles[1], {VertexHandles[0]});
		VertexShouldOnlyHaveEdgesTo(VertexHandles[2], {VertexHandles[0], VertexHandles[3]});
		VertexShouldOnlyHaveEdgesTo(VertexHandles[3], {VertexHandles[2]});
	}

	{
		FEdgeSpecifier Params1{ VertexHandles[0], VertexHandles[1] };
		FEdgeSpecifier Params2{ VertexHandles[3], VertexHandles[2] };
		FEdgeSpecifier Params3{ VertexHandles[2], VertexHandles[0] };
		Graph->CreateBulkEdges({ Params1, Params2, Params3 });
		CHECK(Graph->NumIslands() == 1);

		VerifyEdges({ Params1, Params2, Params3 }, true);
		VertexShouldOnlyHaveEdgesTo(VertexHandles[0], {VertexHandles[1], VertexHandles[2]});
		VertexShouldOnlyHaveEdgesTo(VertexHandles[1], {VertexHandles[0]});
		VertexShouldOnlyHaveEdgesTo(VertexHandles[2], {VertexHandles[0], VertexHandles[3]});
		VertexShouldOnlyHaveEdgesTo(VertexHandles[3], {VertexHandles[2]});
	}
}

class FScopedVertexEdgeCleanupChecker
{
public:
	template<typename TIterable>
	FScopedVertexEdgeCleanupChecker(UGraph* InGraph, const TIterable& InVertexHandles)
	: Graph(InGraph)
	{
		REQUIRE(Graph != nullptr);

		for (const FGraphVertexHandle& VertexHandle : InVertexHandles)
		{
			UGraphVertex* Vertex = VertexHandle.GetVertex();
			REQUIRE(Vertex != nullptr);

			Vertex->ForEachAdjacentVertex(
				[this, &VertexHandle](const FGraphVertexHandle& NeighborHandle)
				{
					EdgeHandles.Add(FEdgeSpecifier{VertexHandle, NeighborHandle});
				}
			);
		}
	}

	~FScopedVertexEdgeCleanupChecker()
	{
		for (const FEdgeSpecifier& EdgeHandle : EdgeHandles)
		{
			if (UGraphVertex* V1 = EdgeHandle.GetVertexHandle1().GetVertex())
			{
				CHECK(V1->HasEdgeTo(EdgeHandle.GetVertexHandle2()) == false);
			}

			if (UGraphVertex* V2 = EdgeHandle.GetVertexHandle2().GetVertex())
			{
				CHECK(V2->HasEdgeTo(EdgeHandle.GetVertexHandle1()) == false);
			}
		}
	}

private:
	UGraph* Graph;
	TArray<FEdgeSpecifier> EdgeHandles;
};

class FScopedVertexIslandCleanupChecker
{
public:
	template<typename TIterable>
	FScopedVertexIslandCleanupChecker(UGraph* InGraph, const TIterable& InVertexHandles)
	: Graph(InGraph)
	{
		REQUIRE(Graph != nullptr);

		for (const FGraphVertexHandle& VertexHandle : InVertexHandles)
		{
			UGraphVertex* Vertex = VertexHandle.GetVertex();
			REQUIRE(Vertex != nullptr);

			FGraphIslandHandle ParentIslandHandle = Vertex->GetParentIsland();
			CHECK(Graph->GetIslands().Contains(ParentIslandHandle) == true);
			CHECK(ParentIslandHandle.IsComplete() == true);
			IslandHandles.Add(ParentIslandHandle);
		}
	}

	~FScopedVertexIslandCleanupChecker()
	{
		for (const FGraphIslandHandle& IslandHandle : IslandHandles)
		{
			CHECK(Graph->GetIslands().Contains(IslandHandle) == false);
			CHECK(IslandHandle.IsComplete() == false);
		}
	}

private:
	UGraph* Graph;
	TArray<FGraphIslandHandle> IslandHandles;
};

TEST_CASE_METHOD(FTestGraphBuilder, "Graph::Remove Vertex::Single::Fully Connected", "[graph]")
{
	PopulateVertices(5, true);
	BuildFullyConnectedEdges(5);

	CHECK(Graph->NumVertices() == 5);
	CHECK(Graph->NumIslands() == 1);

	{
		FScopedVertexEdgeCleanupChecker Checker { Graph, TArray{ VertexHandles[0] } };
		Graph->RemoveVertex(VertexHandles[0]);
	}

	CHECK(VertexHandles[0].IsComplete() == false);
	CHECK(Graph->GetCompleteNodeHandle(VertexHandles[0]).IsComplete() == false);
	CHECK(Graph->GetVertices().Contains(VertexHandles[0]) == false);
	CHECK(Graph->NumVertices() == 4);
	CHECK(Graph->NumIslands() == 1);

	{
		FScopedVertexEdgeCleanupChecker Checker { Graph, TArray{ VertexHandles[1] } };
		Graph->RemoveVertex(VertexHandles[1]);
	}

	CHECK(VertexHandles[1].IsComplete() == false);
	CHECK(Graph->GetCompleteNodeHandle(VertexHandles[1]).IsComplete() == false);
	CHECK(Graph->GetVertices().Contains(VertexHandles[1]) == false);
	CHECK(Graph->NumVertices() == 3);
	CHECK(Graph->NumIslands() == 1);

	{
		FScopedVertexEdgeCleanupChecker Checker { Graph, TArray{ VertexHandles[2] } };
		Graph->RemoveVertex(VertexHandles[2]);
	}

	CHECK(VertexHandles[2].IsComplete() == false);
	CHECK(Graph->GetCompleteNodeHandle(VertexHandles[2]).IsComplete() == false);
	CHECK(Graph->GetVertices().Contains(VertexHandles[2]) == false);
	CHECK(Graph->NumVertices() == 2);
	CHECK(Graph->NumIslands() == 1);

	{
		FScopedVertexEdgeCleanupChecker Checker { Graph, TArray{ VertexHandles[3] } };
		Graph->RemoveVertex(VertexHandles[3]);
	}

	CHECK(VertexHandles[3].IsComplete() == false);
	CHECK(Graph->GetCompleteNodeHandle(VertexHandles[3]).IsComplete() == false);
	CHECK(Graph->GetVertices().Contains(VertexHandles[3]) == false);
	CHECK(Graph->NumVertices() == 1);
	CHECK(Graph->NumIslands() == 1);

	{
		FScopedVertexEdgeCleanupChecker Checker { Graph, TArray{ VertexHandles[4] } };
		Graph->RemoveVertex(VertexHandles[4]);
	}

	CHECK(VertexHandles[4].IsComplete() == false);
	CHECK(Graph->GetCompleteNodeHandle(VertexHandles[4]).IsComplete() == false);
	CHECK(Graph->GetVertices().Contains(VertexHandles[4]) == false);
	CHECK(Graph->NumVertices() == 0);
	CHECK(Graph->NumIslands() == 0);
}

TEST_CASE_METHOD(FTestGraphBuilder, "Graph::Remove Vertex::Single::Linear", "[graph]")
{
	PopulateVertices(10, true);
	BuildLinearEdges(10);

	CHECK(Graph->NumVertices() == 10);
	CHECK(Graph->NumIslands() == 1);

	SECTION("First Vertex")
	{
		CHECK(Graph->GetVertices().Contains(VertexHandles[0]) == true);
		CHECK(VertexHandles[0].IsComplete() == true);

		FScopedVertexEdgeCleanupChecker Checker { Graph, TArray{ VertexHandles[0] } };
		Graph->RemoveVertex(VertexHandles[0]);

		CHECK(Graph->GetVertices().Contains(VertexHandles[0]) == false);
		CHECK(VertexHandles[0].IsComplete() == false);

		CHECK(Graph->NumVertices() == 9);
		CHECK(Graph->NumIslands() == 1);
	}

	SECTION("Split Island")
	{
		CHECK(Graph->GetVertices().Contains(VertexHandles[5]) == true);
		CHECK(VertexHandles[5].IsComplete() == true);

		FScopedVertexEdgeCleanupChecker Checker { Graph, TArray{ VertexHandles[5] } };
		Graph->RemoveVertex(VertexHandles[5]);

		CHECK(Graph->GetVertices().Contains(VertexHandles[5]) == false);
		CHECK(VertexHandles[5].IsComplete() == false);

		CHECK(Graph->NumVertices() == 9);
		CHECK(Graph->NumIslands() == 2);
	}
}

TEST_CASE_METHOD(FTestGraphBuilder, "Graph::Remove Vertex::Bulk::Fully Connected", "[graph]")
{
	PopulateVertices(5, true);
	BuildFullyConnectedEdges(5);

	CHECK(Graph->NumVertices() == 5);
	CHECK(Graph->NumIslands() == 1);

	{
		FScopedVertexEdgeCleanupChecker Checker { Graph, TArray{ VertexHandles[0], VertexHandles[1] } };
		Graph->RemoveBulkVertices( { VertexHandles[0], VertexHandles[1] });
	}

	CHECK(Graph->GetCompleteNodeHandle(VertexHandles[0]).IsComplete() == false);
	CHECK(Graph->GetCompleteNodeHandle(VertexHandles[1]).IsComplete() == false);
	CHECK(VertexHandles[0].IsComplete() == false);
	CHECK(VertexHandles[1].IsComplete() == false);
	CHECK(Graph->NumVertices() == 3);
	CHECK(Graph->NumIslands() == 1);

	{
		FScopedVertexEdgeCleanupChecker Checker { Graph, TArray{ VertexHandles[2], VertexHandles[3], VertexHandles[4] } };
		Graph->RemoveBulkVertices( { VertexHandles[2], VertexHandles[3], VertexHandles[4] });
	}

	CHECK(Graph->GetCompleteNodeHandle(VertexHandles[2]).IsComplete() == false);
	CHECK(Graph->GetCompleteNodeHandle(VertexHandles[3]).IsComplete() == false);
	CHECK(Graph->GetCompleteNodeHandle(VertexHandles[4]).IsComplete() == false);
	CHECK(VertexHandles[2].IsComplete() == false);
	CHECK(VertexHandles[3].IsComplete() == false);
	CHECK(VertexHandles[4].IsComplete() == false);
	CHECK(Graph->NumVertices() == 0);
	CHECK(Graph->NumIslands() == 0);
}


TEST_CASE_METHOD(FTestGraphBuilder, "Graph::Remove Vertex::Bulk::Linear", "[graph]")
{
	PopulateVertices(10, true);
	BuildLinearEdges(10);

	CHECK(Graph->NumVertices() == 10);
	CHECK(Graph->NumIslands() == 1);

	TArray<FGraphVertexHandle> VerticesToRemove = { VertexHandles[1], VertexHandles[2], VertexHandles[8] };
	for (const FGraphVertexHandle& VertexHandle : VerticesToRemove)
	{
		CHECK(Graph->GetVertices().Contains(VertexHandle) == true);
		CHECK(VertexHandle.IsComplete() == true);
	}
	FScopedVertexEdgeCleanupChecker Checker { Graph, VerticesToRemove };

	Graph->RemoveBulkVertices(VerticesToRemove);
	for (const FGraphVertexHandle& VertexHandle : VerticesToRemove)
	{
		CHECK(Graph->GetVertices().Contains(VertexHandle) == false);
		CHECK(VertexHandle.IsComplete() == false);
	}

	CHECK(Graph->NumVertices() == 7);
	CHECK(Graph->NumIslands() == 3);
}

TEST_CASE_METHOD(FTestGraphBuilder, "Graph::Remove Vertex::Incomplete Handle", "[graph]")
{
	PopulateVertices(5, true);
	BuildLinearEdges(5);

	FGraphVertexHandle RemoveHandle{ VertexHandles[0].GetUniqueIndex(), nullptr };
	CHECK(Graph->NumVertices() == 5);
	CHECK(Graph->NumIslands() == 1);

	UGraphIsland* Island = IslandHandles[0].GetIsland();
	REQUIRE(Island != nullptr);
	CHECK(Island->Num() == 5);

	Graph->RemoveVertex(RemoveHandle);

	CHECK(Graph->NumVertices() == 4);
	CHECK(Graph->NumIslands() == 1);
	CHECK(Island->Num() == 4);
}

TEST_CASE_METHOD(FTestGraphBuilder, "Graph::Remove Island", "[graph]")
{
	PopulateVertices(10, true);
	BuildFullyConnectedEdges(5);

	CHECK(Graph->NumVertices() == 10);
	CHECK(Graph->NumIslands() == 2);

	{
		CHECK(Graph->GetIslands().Contains(IslandHandles[0]) == true);
		CHECK(IslandHandles[0].IsComplete() == true);
		TSet<FGraphVertexHandle> IslandVertices = IslandHandles[0].GetIsland()->GetVertices();
		FScopedVertexEdgeCleanupChecker EdgeChecker { Graph, IslandVertices };
		FScopedVertexIslandCleanupChecker IslandChecker { Graph, IslandVertices };
		for (const FGraphVertexHandle& VertexHandle : IslandVertices)
		{
			CHECK(Graph->GetCompleteNodeHandle(VertexHandle).IsComplete() == true);
			CHECK(Graph->GetVertices().Contains(VertexHandle) == true);
		}
		Graph->RemoveIsland(IslandHandles[0]);
		for (const FGraphVertexHandle& VertexHandle : IslandVertices)
		{
			CHECK(Graph->GetCompleteNodeHandle(VertexHandle).IsComplete() == false);
			CHECK(Graph->GetVertices().Contains(VertexHandle) == false);
		}

		CHECK(Graph->GetIslands().Contains(IslandHandles[0]) == false);
		CHECK(IslandHandles[0].IsComplete() == false);
	}

	CHECK(Graph->NumVertices() == 5);
	CHECK(Graph->NumIslands() == 1);

	{
		CHECK(Graph->GetIslands().Contains(IslandHandles[1]) == true);
		CHECK(IslandHandles[1].IsComplete() == true);
		TSet<FGraphVertexHandle> IslandVertices = IslandHandles[1].GetIsland()->GetVertices();
		FScopedVertexEdgeCleanupChecker Checker { Graph, IslandVertices };
		FScopedVertexIslandCleanupChecker IslandChecker { Graph, IslandVertices };
		for (const FGraphVertexHandle& VertexHandle : IslandVertices)
		{
			CHECK(Graph->GetCompleteNodeHandle(VertexHandle).IsComplete() == true);
			CHECK(Graph->GetVertices().Contains(VertexHandle) == true);
		}
		Graph->RemoveIsland(IslandHandles[1]);
		for (const FGraphVertexHandle& VertexHandle : IslandVertices)
		{
			CHECK(Graph->GetCompleteNodeHandle(VertexHandle).IsComplete() == false);
			CHECK(Graph->GetVertices().Contains(VertexHandle) == false);
		}

		CHECK(Graph->GetIslands().Contains(IslandHandles[1]) == false);
		CHECK(IslandHandles[1].IsComplete() == false);
	}

	CHECK(Graph->NumVertices() == 0);
	CHECK(Graph->NumIslands() == 0);
}

TEST_CASE_METHOD(FTestGraphBuilder, "Graph::Event::Vertex Created", "[graph]")
{
	FGraphUniqueIndex Node1Index = { FGuid { 1, 1, 1, 1 } };

	FGraphVertexHandle CreatedHandle;
	Graph->OnVertexCreated.AddLambda(
		[&CreatedHandle, &Node1Index](const FGraphVertexHandle& InCreatedHandle)
		{
			CHECK(InCreatedHandle.IsComplete());
			CHECK(InCreatedHandle.GetUniqueIndex() == Node1Index);
			CreatedHandle = InCreatedHandle;
		}
	);

	FGraphVertexHandle Node1 = Graph->CreateVertex(Node1Index);
	CHECK(CreatedHandle == Node1);
	CHECK(CreatedHandle.IsComplete() == true);
}

TEST_CASE_METHOD(FTestGraphBuilder, "Graph::Event::Island Created", "[graph]")
{
	PopulateVertices(10, false);

	FEdgeSpecifier Params{ VertexHandles[1], VertexHandles[0] };

	FGraphIslandHandle CreatedHandle;
	Graph->OnIslandCreated.AddLambda(
		[&CreatedHandle](const FGraphIslandHandle& InCreatedHandle)
		{
			CHECK(InCreatedHandle.IsComplete());
			CreatedHandle = InCreatedHandle;
		}
	);

	Graph->CreateBulkEdges({ Params });
	CHECK(CreatedHandle.IsComplete());

	UGraphVertex* Vertex1 = VertexHandles[1].GetVertex();
	REQUIRE(Vertex1 != nullptr);
	CHECK(Vertex1->GetParentIsland() == CreatedHandle);

	UGraphVertex* Vertex0 = VertexHandles[0].GetVertex();
	REQUIRE(Vertex0 != nullptr);
	CHECK(Vertex0->GetParentIsland() == CreatedHandle);
}