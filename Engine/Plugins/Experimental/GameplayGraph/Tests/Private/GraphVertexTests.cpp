// Copyright Epic Games, Inc. All Rights Reserved.
#include "TestHarness.h"
#include "TestGraphBuilder.h"

TEST_CASE_METHOD(FTestGraphBuilder, "Graph::Vertex::Create", "[graph][vertex]")
{
	FGraphUniqueIndex VertexIndex = { FGuid { 1, 1, 1, 1 } };
	FGraphVertexHandle VertexHandle = Graph->CreateVertex(VertexIndex);

	CHECK(VertexHandle.IsValid());
	CHECK(VertexHandle.IsComplete());
	CHECK(VertexHandle.HasElement() == true);
	CHECK(VertexHandle.GetUniqueIndex() == VertexIndex);
	CHECK(VertexHandle.GetGraph() == Graph);

	UGraphVertex* Vertex = VertexHandle.GetVertex();
	REQUIRE(Vertex != nullptr);

	CHECK(Vertex->Handle() == VertexHandle);
	CHECK(Vertex->NumEdges() == 0);
	CHECK(Vertex->GetParentIsland() == FGraphIslandHandle{});

	Graph->FinalizeVertex(VertexHandle);

	CHECK(Vertex->GetParentIsland() == Graph->GetIslands().CreateConstIterator()->Key);
}

TEST_CASE_METHOD(FTestGraphBuilder, "Graph::Vertex::Has Edge To", "[graph][vertex]")
{
	constexpr int32 NumVertices = 7;
	PopulateVertices(NumVertices, true);
	BuildFullyConnectedEdges(NumVertices);

	SECTION("Post Construction")
	{
		for (int32 Index1 = 0; Index1 < NumVertices; ++Index1)
		{
			UGraphVertex* Vertex1 = VertexHandles[Index1].GetVertex();
			REQUIRE(Vertex1 != nullptr);

			for (int32 Index2 = Index1 + 1; Index2 < NumVertices; ++Index2)
			{
				UGraphVertex* Vertex2 = VertexHandles[Index2].GetVertex();
				REQUIRE(Vertex2 != nullptr);

				CHECK(Vertex1->HasEdgeTo(VertexHandles[Index2]) == true);
				CHECK(Vertex1->GetEdges().Contains(VertexHandles[Index2]) == true);

				CHECK(Vertex2->HasEdgeTo(VertexHandles[Index1]) == true);
				CHECK(Vertex2->GetEdges().Contains(VertexHandles[Index1]) == true);
			}
		}
	}

	SECTION("Post remove vertex")
	{
		Graph->RemoveVertex(VertexHandles[0]);
		for (int32 Index1 = 1; Index1 < NumVertices; ++Index1)
		{
			UGraphVertex* Vertex1 = VertexHandles[Index1].GetVertex();
			REQUIRE(Vertex1 != nullptr);

			for (int32 Index2 = Index1 + 1; Index2 < NumVertices; ++Index2)
			{
				UGraphVertex* Vertex2 = VertexHandles[Index2].GetVertex();
				REQUIRE(Vertex2 != nullptr);

				CHECK(Vertex1->HasEdgeTo(VertexHandles[Index2]) == true);
				CHECK(Vertex2->HasEdgeTo(VertexHandles[Index1]) == true);
			}

			CHECK(Vertex1->HasEdgeTo(VertexHandles[0]) == false);
			CHECK(Vertex1->GetEdges().Contains(VertexHandles[0]) == false);
		}
	}
}

TEST_CASE_METHOD(FTestGraphBuilder, "Graph::Vertex::Num Edges", "[graph][vertex]")
{
	constexpr int32 NumVertices = 7;
	PopulateVertices(NumVertices, true);
	BuildFullyConnectedEdges(NumVertices);

	SECTION("Post Construction")
	{
		for (int32 Index1 = 0; Index1 < NumVertices; ++Index1)
		{
			UGraphVertex* Vertex1 = VertexHandles[Index1].GetVertex();
			REQUIRE(Vertex1 != nullptr);
			CHECK(Vertex1->NumEdges() == 6);
		}
	}

	SECTION("Post remove vertex")
	{
		Graph->RemoveVertex(VertexHandles[0]);
		for (int32 Index1 = 1; Index1 < NumVertices; ++Index1)
		{
			UGraphVertex* Vertex1 = VertexHandles[Index1].GetVertex();
			REQUIRE(Vertex1 != nullptr);
			CHECK(Vertex1->NumEdges() == 5);
		}
	}
}

TEST_CASE_METHOD(FTestGraphBuilder, "Graph::Vertex::Get Parent Island", "[graph][vertex]")
{
	constexpr int32 NumVertices = 7;
	PopulateVertices(NumVertices, true);
	BuildLinearEdges(NumVertices);

	SECTION("Post Construction")
	{
		for (int32 Index1 = 1; Index1 < NumVertices; ++Index1)
		{
			UGraphVertex* Vertex0 = VertexHandles[Index1-1].GetVertex();
			REQUIRE(Vertex0 != nullptr);
			UGraphVertex* Vertex1 = VertexHandles[Index1].GetVertex();
			REQUIRE(Vertex1 != nullptr);
			CHECK(Vertex0->GetParentIsland() == Vertex1->GetParentIsland());
			CHECK(Vertex0->GetParentIsland() == Graph->GetIslands().CreateConstIterator()->Key);
		}
	}

	SECTION("Post remove vertex")
	{
		Graph->RemoveVertex(VertexHandles[3]);
		for (int32 Index1 = 1; Index1 < 3; ++Index1)
		{
			UGraphVertex* Vertex0 = VertexHandles[Index1-1].GetVertex();
			REQUIRE(Vertex0 != nullptr);
			UGraphVertex* Vertex1 = VertexHandles[Index1].GetVertex();
			REQUIRE(Vertex1 != nullptr);
			CHECK(Vertex0->GetParentIsland() == Vertex1->GetParentIsland());
		}

		{
			UGraphVertex* Vertex0 = VertexHandles[2].GetVertex();
			REQUIRE(Vertex0 != nullptr);
			UGraphVertex* Vertex1 = VertexHandles[4].GetVertex();
			REQUIRE(Vertex1 != nullptr);
			CHECK(Vertex0->GetParentIsland() != Vertex1->GetParentIsland());
		}

		for (int32 Index1 = 5; Index1 < NumVertices; ++Index1)
		{
			UGraphVertex* Vertex0 = VertexHandles[Index1-1].GetVertex();
			REQUIRE(Vertex0 != nullptr);
			UGraphVertex* Vertex1 = VertexHandles[Index1].GetVertex();
			REQUIRE(Vertex1 != nullptr);
			CHECK(Vertex0->GetParentIsland() == Vertex1->GetParentIsland());
		}
	}
}

TEST_CASE_METHOD(FTestGraphBuilder, "Graph::Vertex::For Each Adjacent::With Edge", "[graph][vertex]")
{
	constexpr int32 NumVertices = 7;
	PopulateVertices(NumVertices, true);
	BuildFullyConnectedEdges(NumVertices);

	for (int32 Index1 = 0; Index1 < NumVertices; ++Index1)
	{
		UGraphVertex* Vertex1 = VertexHandles[Index1].GetVertex();
		REQUIRE(Vertex1 != nullptr);

		TSet<FGraphVertexHandle> ConnectedVertices;
		ConnectedVertices.Reserve(NumVertices - 1);

		Vertex1->ForEachAdjacentVertex(
			[&ConnectedVertices](const FGraphVertexHandle& NeighborHandle)
			{
				ConnectedVertices.Add(NeighborHandle);
			}
		);

		CHECK(ConnectedVertices.Num() == Vertex1->GetEdges().Num());
		for (const FGraphVertexHandle& Adjacent : Vertex1->GetEdges())
		{
			CHECK(ConnectedVertices.Contains(Adjacent) == true);
		}

		CHECK(ConnectedVertices.Num() == NumVertices - 1);
		for (int32 Index2 = 0; Index2 < NumVertices; ++Index2)
		{
			if (Index1 == Index2)
			{
				continue;
			}
			
			CHECK(ConnectedVertices.Contains(VertexHandles[Index2]) == true);
			CHECK(Vertex1->GetEdges().Contains(VertexHandles[Index2]) == true);
			ConnectedVertices.Remove(VertexHandles[Index2]);
		}
		CHECK(ConnectedVertices.Num() == 0);
	}
}

TEST_CASE_METHOD(FTestGraphBuilder, "Graph::Vertex::For Each Adjacent::Without Edge", "[graph][vertex]")
{
	constexpr int32 NumVertices = 7;
	PopulateVertices(NumVertices, true);
	BuildFullyConnectedEdges(NumVertices);

	for (int32 Index1 = 0; Index1 < NumVertices; ++Index1)
	{
		UGraphVertex* Vertex1 = VertexHandles[Index1].GetVertex();
		REQUIRE(Vertex1 != nullptr);

		TSet<FGraphVertexHandle> ConnectedVertices;
		ConnectedVertices.Reserve(NumVertices - 1);

		Vertex1->ForEachAdjacentVertex(
			[&ConnectedVertices](const FGraphVertexHandle& NeighborHandle)
			{
				ConnectedVertices.Add(NeighborHandle);
			}
		);

		CHECK(ConnectedVertices.Num() == NumVertices - 1);
		for (int32 Index2 = 0; Index2 < NumVertices; ++Index2)
		{
			if (Index1 == Index2)
			{
				continue;
			}
			
			CHECK(ConnectedVertices.Contains(VertexHandles[Index2]));
			ConnectedVertices.Remove(VertexHandles[Index2]);
		}
		CHECK(ConnectedVertices.Num() == 0);
	}
}

TEST_CASE_METHOD(FTestGraphBuilder, "Graph::Vertex::Event::Remove", "[graph][vertex]")
{
	constexpr int32 NumVertices = 7;
	PopulateVertices(NumVertices, false);

	TArray<bool> VertexRemoved;
	VertexRemoved.Reserve(NumVertices);

	int32 Index = 0;
	for (const FGraphVertexHandle& VertexHandle : VertexHandles)
	{
		VertexRemoved.Add(false);

		UGraphVertex* Vertex = VertexHandle.GetVertex();
		REQUIRE(Vertex != nullptr);
		Vertex->OnVertexRemoved.AddLambda(
			[Vertex, Index, &VertexRemoved](const FGraphVertexHandle& Handle)
			{
				CHECK(Vertex->Handle() == Handle);
				VertexRemoved[Index] = true;
			}
		);

		++Index;
	}

	for (Index = 0; Index < NumVertices; ++Index)
	{
		Graph->RemoveVertex(VertexHandles[Index]);

		for (int32 Index2 = 0; Index2 < NumVertices; ++Index2)
		{
			if (Index2 <= Index)
			{
				CHECK(VertexRemoved[Index2] == true);
			}
			else
			{
				CHECK(VertexRemoved[Index2] == false);
			}
		}
	}
}

TEST_CASE_METHOD(FTestGraphBuilder, "Graph::Vertex::Event::Parent Island Set", "[graph][vertex]")
{
	constexpr int32 NumVertices = 7;
	PopulateVertices(NumVertices, false);

	TArray<FGraphIslandHandle> VertexIslandSet;
	VertexIslandSet.Reserve(NumVertices);

	int32 Index = 0;
	for (const FGraphVertexHandle& VertexHandle : VertexHandles)
	{
		VertexIslandSet.Add({});

		UGraphVertex* Vertex = VertexHandle.GetVertex();
		REQUIRE(Vertex != nullptr);
		Vertex->OnParentIslandSet.AddLambda(
			[Vertex, Index, &VertexIslandSet](const FGraphVertexHandle& VertexHandle, const FGraphIslandHandle& IslandHandle)
			{
				CHECK(Vertex->Handle() == VertexHandle);
				VertexIslandSet[Index] = IslandHandle;
			}
		);

		++Index;
	}

	for (Index = 1; Index < NumVertices; ++Index)
	{
		FEdgeSpecifier Params{ VertexHandles[Index - 1], VertexHandles[Index] };
		Graph->CreateBulkEdges({ Params });
		VerifyEdges({ Params }, true);

		for (int32 Index2 = 0; Index2 < NumVertices; ++Index2)
		{
			UGraphVertex* Vertex = VertexHandles[Index2].GetVertex();
			REQUIRE(Vertex != nullptr);
			CHECK(Vertex->GetParentIsland() == VertexIslandSet[Index2]);

			if (Index2 <= Index)
			{
				CHECK(VertexIslandSet[Index2].IsComplete() == true);
			}
			else
			{
				CHECK(VertexIslandSet[Index2].IsComplete() == false);
			}
		}
	}
}