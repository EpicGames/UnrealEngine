// Copyright Epic Games, Inc. All Rights Reserved.
#include "TestGraphBuilder.h"

#include "TestHarness.h"

FTestGraphBuilder::FTestGraphBuilder()
{
	Graph = NewObject<UGraph>();
	Graph->InitializeFromProperties(FGraphProperties());
}

FTestGraphBuilder::~FTestGraphBuilder()
{
	GraphSanityCheck();
}

void FTestGraphBuilder::PopulateVertices(uint32 Total, bool bFinalize)
{
	for (uint32 Count = 1; Count <= Total; ++Count)
	{
		FGraphUniqueIndex NodeIndex = { FGuid { 0, 0, 0, Count } };
		FGraphVertexHandle Node = Graph->CreateVertex(NodeIndex);
		VertexHandles.Add(Node);
	}

	if (bFinalize)
	{
		FinalizeVertices();
	}
}

void FTestGraphBuilder::FinalizeVertices()
{
	for (uint32 Count = 0; Count < static_cast<uint32>(VertexHandles.Num()); ++Count)
	{
		Graph->FinalizeVertex(VertexHandles[Count]);
	}
}

void FTestGraphBuilder::BuildFullyConnectedEdges(int32 NodesPerIsland)
{
	for (int32 NodeIndex = 0; NodeIndex < VertexHandles.Num(); NodeIndex += NodesPerIsland)
	{
		TArray<FEdgeSpecifier> AllEdges;
		for (int32 SourceOffset = 0; SourceOffset < NodesPerIsland; ++SourceOffset)
		{
			if ((NodeIndex + SourceOffset) >= VertexHandles.Num())
			{
				break;
			}

			for (int32 DestOffset = SourceOffset + 1; DestOffset < NodesPerIsland; ++DestOffset)
			{
				if ((NodeIndex + DestOffset) >= VertexHandles.Num())
				{
					break;
				}

				FEdgeSpecifier Params{ VertexHandles[NodeIndex + SourceOffset], VertexHandles[NodeIndex + DestOffset] };
				AllEdges.Add(Params);
			}
		}

		TArray<FEdgeSpecifier> AllVerifyEdges = AllEdges;
		Graph->CreateBulkEdges(MoveTemp(AllEdges));
		VerifyEdges(AllVerifyEdges, true);
	}

	FinalizeEdges();
}

void FTestGraphBuilder::BuildLinearEdges(int32 NodesPerIsland)
{
	for (int32 NodeIndex = 0; NodeIndex < VertexHandles.Num(); NodeIndex += NodesPerIsland)
	{
		TArray<FEdgeSpecifier> AllEdges;
		for (int32 SourceOffset = 1; SourceOffset < NodesPerIsland; ++SourceOffset)
		{
			if ((NodeIndex + SourceOffset) >= VertexHandles.Num())
			{
				break;
			}

			FEdgeSpecifier Params{VertexHandles[NodeIndex + SourceOffset - 1], VertexHandles[NodeIndex + SourceOffset]};
			AllEdges.Add(Params);
		}

		TArray<FEdgeSpecifier> AllVerifyEdges = AllEdges;
		Graph->CreateBulkEdges(MoveTemp(AllEdges));
		VerifyEdges(AllVerifyEdges, true);
	}

	FinalizeEdges();
}

void FTestGraphBuilder::FinalizeEdges()
{
	IslandHandles.Empty();
	for (const TPair<FGraphIslandHandle, TObjectPtr<UGraphIsland>>& Pair : Graph->GetIslands())
	{
		IslandHandles.Add(Pair.Key);
	}
}

TArray<FEdgeSpecifier> FTestGraphBuilder::GetEdgesForVertex(const FGraphVertexHandle& Handle) const
{
	TArray<FEdgeSpecifier> Edges;
	if (UGraphVertex* Vertex = Handle.GetVertex())
	{
		Edges.Reserve(Vertex->NumEdges());
		Vertex->ForEachAdjacentVertex(
			[&Edges, &Handle](const FGraphVertexHandle& NeighborVertexHandle)
			{
				Edges.Add(FEdgeSpecifier{Handle, NeighborVertexHandle});
			}
		);
	}
	return Edges;
}

void FTestGraphBuilder::GraphSanityCheck() const
{
	for (const TPair<FGraphVertexHandle, TObjectPtr<UGraphVertex>>& Data : Graph->GetVertices())
	{
		CHECK(Data.Key.IsComplete() == true);
		CHECK(Data.Value != nullptr);
		CHECK(Data.Value == Data.Key.GetVertex());
		CHECK(Data.Key == Data.Value->Handle());
		CHECK(Data.Key.GetUniqueIndex().IsTemporary() == false);
		CHECK(Data.Key.GetGraph() == Graph);

		if (Data.Value->GetParentIsland().IsValid())
		{
			CHECK(Data.Value->GetParentIsland().IsComplete());
			CHECK(Data.Value->GetParentIsland().GetIsland() == Graph->GetIslands().FindRef(Data.Value->GetParentIsland()));
		}

		Data.Value->ForEachAdjacentVertex(
			[this, &Data](const FGraphVertexHandle& NeighborVertexHandle)
			{
				CHECK(NeighborVertexHandle.IsComplete() == true);
				CHECK(NeighborVertexHandle.GetVertex() == Graph->GetVertices().FindRef(NeighborVertexHandle));

				const UGraphVertex* AdjacentVertex = NeighborVertexHandle.GetVertex();
				REQUIRE(AdjacentVertex != nullptr);

				CHECK(Data.Value->HasEdgeTo(NeighborVertexHandle) == true);
				CHECK(AdjacentVertex->HasEdgeTo(Data.Key) == true);
			}
		);
	}

	for (const TPair<FGraphIslandHandle, TObjectPtr<UGraphIsland>>& Data : Graph->GetIslands())
	{
		CHECK(Data.Key.IsComplete() == true);
		CHECK(Data.Value != nullptr);
		CHECK(Data.Value == Data.Key.GetIsland());
		CHECK(Data.Key == Data.Value->Handle());
		CHECK(Data.Key.GetUniqueIndex().IsTemporary() == false);
		CHECK(Data.Key.GetGraph() == Graph);

		for (const FGraphVertexHandle& VertexHandle : Data.Value->GetVertices())
		{
			CHECK(VertexHandle.IsComplete() == true);
			CHECK(VertexHandle.GetVertex() == Graph->GetVertices().FindRef(VertexHandle));
		}

		IslandVertexParentIslandSanityCheck(Data.Key);
	}
}

void FTestGraphBuilder::IslandVertexParentIslandSanityCheck(const FGraphIslandHandle& IslandHandle) const
{
	UGraphIsland* Island = IslandHandle.GetIsland();
	REQUIRE(Island != nullptr);

	Island->ForEachVertex(
		[&IslandHandle](const FGraphVertexHandle& VertexHandle)
		{
			CHECK(VertexHandle.GetVertex()->GetParentIsland() == IslandHandle);
		}
	);
}

void FTestGraphBuilder::VerifyEdges(const TArray<FEdgeSpecifier>& Edges, bool bExist) const
{
	for (const FEdgeSpecifier& Edge : Edges)
	{
		UGraphVertex* V1 = Edge.GetVertexHandle1().GetVertex();
		REQUIRE(V1 != nullptr);

		UGraphVertex* V2 = Edge.GetVertexHandle2().GetVertex();
		REQUIRE(V2 != nullptr);

		CHECK(V1->HasEdgeTo(V2->Handle()) == bExist);
		CHECK(V2->HasEdgeTo(V1->Handle()) == bExist);
	}
}

void FTestGraphBuilder::VertexShouldOnlyHaveEdgesTo(const FGraphVertexHandle& Source, const TArray<FGraphVertexHandle>& Targets) const
{
	UGraphVertex* SourceVertex = Source.GetVertex();
	REQUIRE(SourceVertex != nullptr);
	CHECK(SourceVertex->NumEdges() == Targets.Num());
	for (const FGraphVertexHandle& AdjacentHandle : Targets)
	{
		CHECK(SourceVertex->HasEdgeTo(AdjacentHandle) == true);
		
		UGraphVertex* AdjacentVertex = AdjacentHandle.GetVertex();
		REQUIRE(AdjacentVertex != nullptr);
		CHECK(AdjacentVertex->HasEdgeTo(Source) == true);
	}
}