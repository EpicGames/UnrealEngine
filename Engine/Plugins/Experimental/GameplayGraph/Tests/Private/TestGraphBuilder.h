// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Graph/Graph.h"

class FTestGraphBuilder
{
public:
	FTestGraphBuilder();
	~FTestGraphBuilder();

	void PopulateVertices(uint32 Total, bool bFinalize);
	void FinalizeVertices();

	void BuildFullyConnectedEdges(int32 NodesPerIsland);
	void BuildLinearEdges(int32 NodesPerIsland);

	void FinalizeEdges();

	TArray<FEdgeSpecifier> GetEdgesForVertex(const FGraphVertexHandle& Handle) const;

protected:
	TObjectPtr<UGraph> Graph;
	TArray<FGraphVertexHandle> VertexHandles;
	TArray<FGraphIslandHandle> IslandHandles;

	void GraphSanityCheck() const;
	void IslandVertexParentIslandSanityCheck(const FGraphIslandHandle& IslandHandle) const;
	void VerifyEdges(const TArray<FEdgeSpecifier>& Edges, bool bExist) const;
	void VertexShouldOnlyHaveEdgesTo(const FGraphVertexHandle& Source, const TArray<FGraphVertexHandle>& Targets) const;
};