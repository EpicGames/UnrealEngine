// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Graph/GraphElement.h"

#include "TestGraphElement.generated.h"

UCLASS()
class UTestGraphElement : public UGraphElement
{
	GENERATED_BODY()

public:
	void PublicSetUniqueIndex(FGraphUniqueIndex InUniqueIndex)
	{
		SetUniqueIndex(InUniqueIndex);
	}

	FGraphUniqueIndex PublicGetUniqueIndex()
	{
		return GetUniqueIndex();
	}


	void PublicSetParentGraph(TObjectPtr<UGraph> InGraph)
	{
		SetParentGraph(InGraph);
	}

	TObjectPtr<UGraph> PublicGetGraph() const
	{
		return GetGraph();
	}
};
