// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedPlatforms(UnrealPlatformClass.All)]
public class GlobalConfigurationDataTestsTarget : TestTargetRules
{
	public GlobalConfigurationDataTestsTarget(TargetInfo Target) : base(Target)
	{
	}
}
