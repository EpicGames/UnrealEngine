// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestHarness.h"

#include "GlobalConfigurationData.h"
#include "GlobalConfigurationRouter.h"
#include "GlobalConfigurationTestData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GlobalConfigurationTestData)

class FGlobalConfigurationTestRouter : public IGlobalConfigurationRouter
{
public:
	FGlobalConfigurationTestRouter() : IGlobalConfigurationRouter("Test", 0)
	{
	}

	virtual TSharedPtr<FJsonValue> TryGetDataFromRouter(const FString& Name) const override
	{
		if (const TSharedRef<FJsonValue>* Value = Data.Find(Name))
		{
			return *Value;
		}
		return {};
	}

	virtual void GetAllDataFromRouter(TMap<FString, TSharedRef<FJsonValue>>& DataOut) const override
	{
		DataOut = Data;
	}

	void AddValue(const FStringView& Key, const FStringView& Value)
	{
		Data.Add(FString(Key), IGlobalConfigurationRouter::TryParseString(Value).ToSharedRef());
	}

	TMap<FString, TSharedRef<FJsonValue>> Data;
};

using namespace UE::GlobalConfigurationData;

TEST_CASE("GlobalConfigurationTest", "All")
{
	FGlobalConfigurationTestRouter TestRouter;

	TestRouter.AddValue(TEXT("BoolTrueValue"), TEXT("true"));
	TestRouter.AddValue(TEXT("BoolFalseValue"), TEXT("false"));

	TestRouter.AddValue(TEXT("IntValue"), TEXT("1"));
	TestRouter.AddValue(TEXT("FloatValue"), TEXT("1.5"));
	TestRouter.AddValue(TEXT("StringValue"), TEXT("foobar"));

	SECTION("Find Data")
	{
		bool bBoolValue = false;
		int32 IntValue = 0;
		float FloatValue = 0;
		FString StringValue;

		CHECK(TryGetData(TEXT("BoolTrueValue"), bBoolValue));
		CHECK(bBoolValue == true);
		CHECK(TryGetData(TEXT("BoolFalseValue"), bBoolValue));
		CHECK(bBoolValue == false);
		CHECK(TryGetData(TEXT("IntValue"), IntValue));
		CHECK(IntValue == 1);
		CHECK(TryGetData(TEXT("FloatValue"), FloatValue));
		CHECK(FMath::IsNearlyEqual(FloatValue, 1.5f));
		CHECK(TryGetData(TEXT("StringValue"), StringValue));
		CHECK(StringValue == TEXT("foobar"));
	}

	SECTION("Missing Data")
	{
		bool bBoolValue = false;
		int32 IntValue = 0;
		float FloatValue = 0;
		FString StringValue;

		CHECK(TryGetData(TEXT("Gobble"), bBoolValue) == false);
		CHECK(bBoolValue == false);
		CHECK(TryGetData(TEXT("Gooble"), IntValue) == false);
		CHECK(IntValue == 0);
		CHECK(TryGetData(TEXT("Gooooo"), FloatValue) == false);
		CHECK(FloatValue == 0);
		CHECK(TryGetData(TEXT("Gowowow"), StringValue) == false);
		CHECK(StringValue.IsEmpty());
	}

	SECTION("Default Data")
	{
		CHECK(GetDataWithDefault(TEXT("BoolTrueValue"), false) == true);
		CHECK(GetDataWithDefault(TEXT("BoolFalseValue"), true) == false);
		CHECK(GetDataWithDefault<int32>(TEXT("IntValue"), 0) == 1);
		CHECK(FMath::IsNearlyEqual(GetDataWithDefault(TEXT("FloatValue"), 0.f), 1.5f));
		CHECK(GetDataWithDefault<FString>(TEXT("StringValue"), {}) == TEXT("foobar"));

		CHECK(GetDataWithDefault(TEXT("Garbage"), false) == false);
		CHECK(GetDataWithDefault(TEXT("Garbage"), true) == true);
		CHECK(GetDataWithDefault<int32>(TEXT("Garbage"), 123) == 123);
		CHECK(FMath::IsNearlyEqual(GetDataWithDefault(TEXT("Garbage"), 5.f), 5.f));
		CHECK(GetDataWithDefault<FString>(TEXT("Garbage"), {}) == FString());
	}
	
	SECTION("Data Conversions")
	{
		CHECK(GetDataWithDefault(TEXT("IntValue"), false) == true);
		CHECK(FMath::IsNearlyEqual(GetDataWithDefault(TEXT("IntValue"), 0.f), 1.f));
	}

	TestRouter.AddValue(TEXT("ObjectValue"), TEXT("{\"bBoolValue\":true,\"IntValue\":5,\"IntValueArray\":[1,2,3]}"));

	SECTION("UStructs")
	{
		FGlobalConfigurationTestStruct TestStruct;
		CHECK(TryGetData(TEXT("ObjectValue"), TestStruct));
		CHECK(TestStruct.bBoolValue == true);
		CHECK(TestStruct.IntValue == 5);
		CHECK(TestStruct.IntValueArray.Num() == 3);
		if (TestStruct.IntValueArray.Num() == 3)
		{
			CHECK(TestStruct.IntValueArray[0] == 1);
			CHECK(TestStruct.IntValueArray[1] == 2);
			CHECK(TestStruct.IntValueArray[2] == 3);
		}
	}

	SECTION("UObjects")
	{
		UGlobalConfigurationTestObject* TestObject = NewObject<UGlobalConfigurationTestObject>();

		CHECK(TryGetData(TEXT("ObjectValue"), TestObject));
		CHECK(TestObject->bBoolValue == true);
		CHECK(TestObject->IntValue == 5);
		CHECK(TestObject->IntValueArray.Num() == 3);
		if (TestObject->IntValueArray.Num() == 3)
		{
			CHECK(TestObject->IntValueArray[0] == 1);
			CHECK(TestObject->IntValueArray[1] == 2);
			CHECK(TestObject->IntValueArray[2] == 3);
		}
	}

	SECTION("SubProperties")
	{
		CHECK(GetDataWithDefault(TEXT("ObjectValue.bBoolValue"), false) == true);
		CHECK(GetDataWithDefault(TEXT("ObjectValue.IntValue"), 0) == 5);
	}
}
