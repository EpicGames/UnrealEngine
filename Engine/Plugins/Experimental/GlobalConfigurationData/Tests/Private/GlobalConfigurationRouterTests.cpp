// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestHarness.h"

#include "GlobalConfigurationData.h"
#include "GlobalConfigurationTestData.h"

#include "HAL/IConsoleManager.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"

using namespace UE::GlobalConfigurationData;

TEST_CASE("GlobalConfigurationRouterTest", "Routers")
{
	SECTION("Config")
	{
		const TCHAR* Section = TEXT("GlobalConfigurationData");

		CHECK(GetDataWithDefault(TEXT("BoolValue"), false) == false);
		CHECK(GetDataWithDefault<int32>(TEXT("IntValue"), 0) == 0);
		CHECK(GetDataWithDefault<FGlobalConfigurationTestStruct>(TEXT("StructValue"), {}).IntValue == 0);

		GConfig->SetString(Section, TEXT("BoolValue"), TEXT("1"), GEngineIni);
		GConfig->SetString(Section, TEXT("IntValue"), TEXT("5"), GEngineIni);
		GConfig->SetString(Section, TEXT("StructValue"), TEXT("{\"bBoolValue\":true,\"IntValue\":5,\"IntValueArray\":[1,2,3]}"), GEngineIni);
		FCoreDelegates::TSOnConfigSectionsChanged().Broadcast(GEngineIni, { Section });

		CHECK(GetDataWithDefault(TEXT("BoolValue"), false) == true);
		CHECK(GetDataWithDefault<int32>(TEXT("IntValue"), 0) == 5);
		CHECK(GetDataWithDefault<FGlobalConfigurationTestStruct>(TEXT("StructValue"), {}).IntValue == 5);

		GConfig->RemoveKey(Section, TEXT("BoolValue"), GEngineIni);
		GConfig->RemoveKey(Section, TEXT("IntValue"), GEngineIni);
		GConfig->RemoveKey(Section, TEXT("StructValue"), GEngineIni);
		FCoreDelegates::TSOnConfigSectionsChanged().Broadcast(GEngineIni, { Section });

		CHECK(GetDataWithDefault(TEXT("BoolValue"), false) == false);
		CHECK(GetDataWithDefault<int32>(TEXT("IntValue"), 0) == 0);
		CHECK(GetDataWithDefault<FGlobalConfigurationTestStruct>(TEXT("StructValue"), {}).IntValue == 0);
	}

	SECTION("Console Command")
	{
		CHECK(GetDataWithDefault(TEXT("BoolValue"), false) == false);
		CHECK(GetDataWithDefault<int32>(TEXT("IntValue"), 0) == 0);
		CHECK(GetDataWithDefault<FGlobalConfigurationTestStruct>(TEXT("StructValue"), {}).IntValue == 0);

		IConsoleManager::Get().ProcessUserConsoleInput(TEXT("GCD.RegisterValue BoolValue 1"), *GLog, nullptr);
		IConsoleManager::Get().ProcessUserConsoleInput(TEXT("GCD.RegisterValue IntValue 5"), *GLog, nullptr);
		IConsoleManager::Get().ProcessUserConsoleInput(TEXT("GCD.RegisterValue StructValue {\"bBoolValue\":true,\"IntValue\":5,\"IntValueArray\":[1,2,3]}"), *GLog, nullptr);

		CHECK(GetDataWithDefault(TEXT("BoolValue"), false) == true);
		CHECK(GetDataWithDefault<int32>(TEXT("IntValue"), 0) == 5);
		CHECK(GetDataWithDefault<FGlobalConfigurationTestStruct>(TEXT("StructValue"), {}).IntValue == 5);

		IConsoleManager::Get().ProcessUserConsoleInput(TEXT("GCD.UnregisterValue BoolValue"), *GLog, nullptr);
		IConsoleManager::Get().ProcessUserConsoleInput(TEXT("GCD.UnregisterValue IntValue"), *GLog, nullptr);
		IConsoleManager::Get().ProcessUserConsoleInput(TEXT("GCD.UnregisterValue StructValue"), *GLog, nullptr);

		CHECK(GetDataWithDefault(TEXT("BoolValue"), false) == false);
		CHECK(GetDataWithDefault<int32>(TEXT("IntValue"), 0) == 0);
		CHECK(GetDataWithDefault<FGlobalConfigurationTestStruct>(TEXT("StructValue"), {}).IntValue == 0);
	}
}
