// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "UObject/Object.h"

#include "GlobalConfigurationTestData.generated.h"

UCLASS()
class UGlobalConfigurationTestObject : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY()
	bool bBoolValue = false;

	UPROPERTY()
	int32 IntValue = 0;

	UPROPERTY()
	TArray<int32> IntValueArray;
};

USTRUCT()
struct FGlobalConfigurationTestStruct
{
	GENERATED_BODY()

public:
	UPROPERTY()
	bool bBoolValue = false;

	UPROPERTY()
	int32 IntValue = 0;

	UPROPERTY()
	TArray<int32> IntValueArray;
};