// Copyright Epic Games, Inc. All Rights Reserved.
#include "TestCommon/Initialization.h"
#include "Modules/ModuleManager.h"

#include <catch2/catch_test_macros.hpp>

GROUP_BEFORE_GLOBAL(Catch::DefaultGroup)
{
	InitAll(true, true);

	FModuleManager::LoadModulePtr<IModuleInterface>(TEXT("GlobalConfigurationDataCore"));
}

GROUP_AFTER_GLOBAL(Catch::DefaultGroup)
{
	CleanupAll();
}
