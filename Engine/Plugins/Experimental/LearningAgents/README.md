# Learning Agents
Learning Agents is a machine learning plugin for AI bots. It allows you to train your NPCs via reinforcement and/or imitation learning. The plugin aims to be useful in the creation of game-playing agents, physics-based animations, automated QA bots, and much more!

For more information, visit the forums:
https://dev.epicgames.com/community/learning/tutorials/8OWY/unreal-engine-learning-agents-introduction

## Feedback
Please post your questions and feedback on the developer forums so everyone can benefit from the discussion. If that is not a good option, you can also reach out to us at learning-agents@epicgames.com
