# Mover Plugin

Mover is an Unreal Engine plugin to support movement of actors with rollback networking, using the Network Prediction Plugin or Chaos Networked Physics. This plugin is the potential successor to Character Movement Component. The goal is to allow gameplay developers to focus on crafting motion without having to be experts in networking.

**The Mover plugin is Experimental. Many features are incomplete or missing. APIs and data formats are subject to change at any time.**


## Getting Started

Add Mover and MoverExamples plugins to any existing project, or from a new Blank template project.

Open the L_CharacterMovementBasics map, and activate Play-in-Editor (PIE).

Recommended project settings to start:
- Network Prediction / Preferred Ticking Policy: Fixed
- Network Prediction / Simulated Proxy Network LOD: Interpolated
- Network Prediction / Enable Fixed Tick Smoothing: true

This will have Mover objects simulating at a fixed rate, which is different than the variable rendering frame rate. The smoothing option provides visual compensation. Without it, you may feel like the movement simulation is rougher than the camera movement or skeletal animation.

## Examples

The MoverExamples plugin provides a collection of maps and actor examples, using a mixture of Blueprint and C++.

Maps of interest:

- **L_CharacterMovementBasics** has a variety of terrain features and movement examples.
- **L_LayeredMoves** is focused on demonstrating many layered move types, with varying options.
- **L_PathfindingMovement** is focused on demonstrating Mover and different AI-driven movement with pathfinding including NavWalking mode.
- **L_PhysicallyBasedCharacter** has an example of a physics-based pawn using ChaosMover and Chaos Networked Physics. See section below for more details.

Pawn/Actor Blueprints of interest:

- **AnimatedMannyPawn** is the simplest character and is based on the UE5 mannequin character used in the engine's template projects.
- **AnimatedMannyPawnExtended** adds a variety of movement capabilities, such as dashing, vaulting, and ziplining.
- **PathFollowingMannyPawn** is set up to support AI-driven pathfollowing as an example of nav-mesh movement.
- **BP_SplineFollowerPlatform** shows a simple moving platform that predictably follows a defined path. This is an Actor, but not a Pawn.

Additional Examples: 

- The **Game Animation Sample** project, available for free through Fab's marketplace, contains more advanced animation techniques and polished examples of controllable characters using the Mover plugin.
- The **Project Titan** example, also free through Fab, uses Mover for more complex and interesting methods of movement, such as gliding, sliding, and a grappling hook.

## Concepts

### Movement Modes

A **movement mode** (`UBaseMovementMode`) is an object that governs how your actor moves through space. It can both generate proposed movement and execute it. For example, walking, falling, climbing, etc. These modes look at inputs, decide how the character should move, and then attempt to actually move it through the world. 

There is always one and only one mode active a time.

The default movement set includes modes: Walking, Falling, Flying, Swimming, and NavWalking. Async variants (AsyncWalking, AsyncFalling, AsyncFlying, AsyncNavWalking) are available for use when configured for multi-threaded movement simulation.

### Layered Moves

**Layered moves** (`FLayeredMoveBase`) represent temporary additional movement. For example, a constant force briefly launching up into the air, a homing force that moves you towards an enemy, or even animation-driven root motion.

They only generate a proposed move, and rely on the active movement mode to execute it. 

Multiple layered moves can be active at a time, and their moves can be mixed with other influences. Their lifetime can have a duration or be instantaneous, as well as surviving transitions between modes.

There is an alternative version of "instanced" layered moves in-progress that will replace the existing ones. The major difference is that the existing layered moves combine logic and data on the same UStruct type. The new version will separate logic and data into a stateless 'logic' UObject, and a data UStruct. This will allow Blueprint extensions and provide a cleaner separation of networked data.

### Instant Movement Effects

**Instant movement effects** (`FInstantMovementEffect`) can affect movement state directly on a Mover-based actor during simulation without consuming time. They are only applied during a simulation tick and then removed.

Multiple instant effects can be queued and will be executed sequentially as soon as possible. These execution windows occur at the beginning and end of each simulation step, including between substeps (for example, if the mode changed from falling to walking halfway through the sim tick).

This could be used for teleporting, applying instantaneous launch velocities, or forcing a movement mode change. 

### Movement Modifiers

**Movement Modifiers** (`FMovementModifierBase`) are used to apply changes that *indirectly influence* the movement simulation, without proposing or executing any movement, but still in sync with the simulation.

Common uses would be for movement "stance" changes like crouching, stealth, etc. or other mechanics requiring a number of settings to change together.

### Transitions

**Transitions** (`UBaseMovementModeTransition`) are objects that evaluate whether a Mover actor should change its mode, based on the current state. It can also trigger side effects when activated. They can set up to only evaluate when a particular mode is active, or they can be global and evaluated regardless of the current mode.

The use of transitions is optional, with other methods of switching modes available.

### Backend Liaisons

A Mover-based actor requires a MoverComponent (an ActorComponent type) to operate. This is the main object that other systems interface with. Unlike many ActorComponents that use TickComponent to drive updating, Mover relies on an external "backend" to drive it.

The liaison acts as an intermediary between the MoverComponent and a backend system, passing function calls or events when it's time to produce input, advance the simulation, etc.


Included backend liaisons:
- **MoverNetworkPredictionLiaisonComponent** - interfaces with the Network Prediction Plugin for network or standalone play
- **MoverStandaloneLiaisonComponent** - a simpler backend only for standalone play without physics-based movement
- The **ChaosMover** plugin has its own Liaison component for use with standalone physics or networked physics play

### Other Concepts

- **Composable Input and Sync State:** Inputs are authored by the controlling owner, and influence a movement simulation step. Sync state is a snapshot that describes a Mover actor's movement at a point in time. Both inputs and sync state can have custom struct data attached to them dynamically, as natively-defined structs or User-Defined Structs in Blueprints.

- **Shared Settings:** Collections of properties that multiple movement objects share, to avoid duplication of settings between decoupled objects. The list is managed by MoverComponent based on which settings classes its modes call for. Your custom modes are not required to use Shared Settings and may approach settings differently, for example having a self-contained set of properties or referencing an external data table.

- **Persistent Sync State Data Types:** There is almost always state data that you want to carry over from the previous frame, especially core movement data such as position.  Adding types to this array allows carry-over of types to automatically happen. Any state data *not* present on this list will have to be added every frame.

- **Input Production:** Mover does not deal directly with Unreal's input systems such as Enhanced Input. Projects instead should translate input events into InputCmd structure(s) that get submitted just before simulation frames, via the ProduceInput interface.  AMoverExamplesCharacter's use of OnProduceInput shows a C++ method. The AnimatedMannyPawnExtended BP in the MoverExamples plugin adds additional inputs via the "On Produce Input" function implementation.

- **Trajectory Prediction:** Mover provides a method to get samples that predict where an actor will end up in the future via the GetPredictedTrajectory function. This is useful in support of advanced animation techniques like Motion Matching.

- **Nav Mover Component:** (optional) This is a component meant to house navigation movement specific settings and functionality. It is designed to live alongside the MoverComponent attached to the same actor.

- **Movement Utility Libraries:** (optional) These are collections of functions useful for crafting movement, typically callable from Blueprints. For example: `UMovementUtils`, `UGroundMovementUtils`, `UAirMovementUtils`, etc.  When implementing the default movement set, we attempted to break the methods into these libraries where possible, so that developers can make use of them in their own movement.

- **Blackboard:** (optional) This is a way for decoupled systems to share information or cache computations between simulation ticks without adding to the size of the networked simulation state. We are in the process of replacing SimBlackboard with Rollback Blackboard, which is both rollback-aware and prediction-friendly.

- **Move Record:** (optional) This is a mechanism for tracking combinations of moves and marking which ones should affect the final state of the moving actor.  For example, a movement that results in a character stepping up onto a short ledge consists of both horizontal and vertical movement, but we don't want the vertical movement to be included when computing the actor's post-move velocity. The record(s) for vertical movement would be marked as non-contributing. 


## Comparing with Character Movement Component (CMC)

- **Movement Modes** are very similar in both systems, but modular in Mover
- **Layered Moves** are similar to Root Motion Sources (RMS)
- **Instant Effects** are a new concept, and are used to implement CMC features like launching and teleporting
- **Movement Modifiers** are a new concept, and are used to implement CMC features like crouching
- **Transitions** are a new concept, as a modular way to drive mode-changing evaluation logic
- Movement from modes and layered moves can be mixed together
- It is easier to add custom movement modes, even from plugins and at runtime
- The DefaultMovementSet in Mover is similar to the modes built in to CharacterMovementComponent (Walking, Falling, Flying, Swimming, NavWalking, etc.)

The Mover plugin provides a DefaultMovementSet that is similar to the modes that CharacterMovementComponent provides. This movement set assumes a similar actor composition, with a Capsule primitive component as the root, with a skeletal mesh attached to it.

MoverComponent does not require your actor class to derive from ACharacter.

MoverComponent requires a root SceneComponent, but it does not have to be a singular vertically-oriented capsule or even a PrimitiveComponent. Developers are free to create Mover actors with no collision primitives if they wish.

MoverComponent does not require or assume a skeletal mesh as the visual representation.

In CMC, adding custom data to be passed between clients and server required subclassing the component and overriding key functions. The Mover plugin allows custom input and state data to be added dynamically at runtime, without customizing the MoverComponent.

Networking Model: 
- In CMC, owning clients send a combination of inputs and state as a "move" at the client's frame rate. The server receives them and performs the same move immediately, then compares state to decide if a correction is needed and replies with either an acknowledgement of the move or a block of corrective state data. 
- In Mover / Network Prediction, all clients and server are attempting to simulate on a shared timeline, with clients running predictively ahead of the server by a small amount of time. Clients author inputs for a specific simulation time/frame, and that is all they send to the server. The server buffers these inputs until their simulation time comes. After performing all movement updates, the server broadcasts state to all clients and the clients decide whether a correction (rollback + resim) is needed.

Unlike CMC, the state of the Mover actor is not directly modifiable externally at any time. For example, there is no Velocity property to directly manipulate. Instead, developers must make use of modes and layered moves to affect change during the next available simulation tick.  Additionally, player-provided inputs such as move input and button presses must be combined into a single Input Command for the movement simulation tick, rather than immediately affecting the Mover actor's state.  Depending on your project settings, you may have player input from several frames contributing to a single movement simulation tick.


## Debugging

### Gameplay Debugger Tool

Visualization and state readout information is available through the Gameplay Debugger Tool (GDT). To activate the gameplay debugger tool, typically via the **'** key, and toggle the Mover category using the NumPad. The locally-controlled player character will be selected by default, but you can change this via GDT input and **gdt.\*** console commands.

### Logging

Output log information coming from the Mover plugin will have the **LogMover** category.

### Console Commands

There are a variety of useful console commands with this prefix: **Mover.\***


## Physics-driven Character Example

Included in MoverExamples is a physics-driven version of the Manny pawn: ChaosMannyPawnExtended.  To try it out, open the L_PhysicallyBasedCharacter map in the MoverExamples plugin. Make sure you adjust your project's settings according to the text inside the map.

Other MoverExamples pawns, and the CharacterMovementComponent, use a "kinematic" movement style where the pawn's shape is moved by testing the surroundings in an ad hoc manner. Responses to physics forces, and interactions with physics-simulated objects, are difficult to implement. With physics driving the movement, the pawn can realistically apply forces to other objects and have them applied right back.

This physics-based character is NOT using the Network Prediction plugin. Instead, it is driven by the Chaos Networked Physics system, which has many similarities in how it operates. Under the hood, a Character Ground Constraint from the physics system handles performing the proposed motion that the Mover system generates. The physics simulation is running at a fixed tick rate, ahead of the game thread. As the simulation progresses, the game thread's character representation interpolates towards the most recent physics state.

Notes:
- Due to the async nature of the physics simulation, there is some additional latency between movement and other gameplay systems. This will affect things like the amount of time between player input occurring and the pawn moving on screen. And when using animation root motion, there may be delay between a character's animation pose and when the motion is seen.
- Some Mover mechanisms will not work with a physics-based character, such as the teleport effect.
- Other physics objects that the character interacts with should be set up for replication. Otherwise this will lead to differences between client and server.
- Interactions with moving non-physics objects will likely show poor results, due to ticking differences between the physics simulation and the rest of the game world.
- Various gameplay events may not be connected or are unreliable. 
- Crafting customized movement may be less flexible without also using a modified physics constraint.
- Physics-driven Mover actors are not well-synchronized with the movement of non-physics actors, such as those manually moved via gameplay scripting (or even Mover actors using NPP).


## FAQ

- **Should my project switch to Mover from CharacterMovementComponent?**  This depends greatly on the scope of your project and will require some due diligence. The Mover plugin is experimental and hasn't gone through the rigors of a shipped project yet. There are many gaps in functionality, and little time has been spent on scaling/performance. Single-player or games with smaller character counts will be more feasible. 

- **Does Mover fix synchronization issues between movement and the Gameplay Ability System?** The short answer is no. GAS still has its own independent replication methods. The use of Network Prediction opens the door for GAS (or other systems) to integrate with Network Prediction and achieve good synchronization with movement.

- **What about single-player games?** Mover is useful for single-player games as well, you just won't be making use of the networking and rollback features. Consider changing the project setting "Network Prediction / Preferred Ticking Policy" to "Independent" mode. This will make the movement simulation tick at the same rate as the game world.  You can even eliminate all NPP overhead by choosing "MoverStandaloneLiaisonComponent" as your "Backend Class" property on your actors' MoverComponents.  Physics-based pawns should also work well for single-player projects.


## Limitations and Known Issues

**See also: Network Prediction Plugin's documentation** Some of the Mover plugin's current limitations come from its dependency on the Network Prediction plugin. Please review its documentation for more info.

**Ticking Ordering:** All Network Prediction simulations tick before Unreal's world tick groups and always in a consistent order, making it difficult to implement mechanics where tight ticking dependencies are needed.

**Fixed Tick Simulation + Variable Engine/Rendering Rate:** When using Fixed Tick simulation with a variable Engine ticking (rendering) rate, you could go several rendered frames in between simulation ticks, or even have multiple simulation ticks occur during a single rendered frame. This has implications for things like input capturing, where you may need to combine several frames of sampled input into an input command for a single simulation tick.

Additionally, when the simulation runs at a lower rate than rendering, the result is rough-looking movement that is unacceptable for most projects. Visual smoothing options are available when using Network Prediction. Set Project Settings' "Network Prediction / Enable Fixed Tick Smoothing" to true, and set your MoverComponent's "Smoothing Mode" to "Visual Component Offset".

**Blueprinting Support:** While Blueprint support has been significantly expanded to include movement modes and custom input & state data, it isn't 100% complete yet. There are certain things that still require native C++ code, or are clunky to implement in Blueprints.

**Instanced Subobjects:** There are known issues with removing and replacing elements of the movement mode map on instanced actors placed in a level, potentially preventing them from moving. Consider other workflows that avoid this kind of customization on level-placed instances.

**Alternate Collision Shapes in the Default Movement Set:** The core MoverComponent tries to make as few requirements as possible on the composition of the actor, but the default movement set has more rigid assumptions. A root shape other than a capsule should work reasonably well, as long as it's symmetrical in the horizontal plane. 

**Forward-Predicted Sim Proxy Characters:**  Characters controlled by other players are typically poor candidates for forward prediction, which relies on past inputs to predict future movement. Acceleration and direction changes, as well as action inputs like jumping, are unpredictable and will be the source of frequent mispredictions that can give the sim proxy character popping or choppy movement. Consider using Interpolated mode for the "Simulated Proxy Network LOD" project setting, which will give smooth results at the cost of some visual latency.

**MoverComponent Defaults Are Currently Focused on Characters:** The current component has more API and data than would be required for Mover actors that are not Character-like. Our goal is to make a minimalist base component with slimmer API and property set, with a specialization for Character-like actors implemented separately as part of the default movement set. 

**Sim Blackboard vs Rollback Blackboard:** Mover is replacing the SimBlackboard in favor of a Rollback-aware Blackboard that will restore prior data as rollbacks occur. It is best practice for movement logic to not assume there's always a valid entry and should have a fallback.

**Multiple Modifiers of the Same Type:** Currently Mover expects there to be only one modifier of a given type active at a time. This can be worked around by overriding the Matches function and using more criteria than just the type, as long as that criteria is reflected by data included in the NetSerialize function.

**Absolute Timestamps in Blackboard (prior to 5.7):** Timestamps were originally kept as single-precision floats, and later updated to double-precision. FMoverTimeStep::BaseSimTimeMs is an example. If you had been storing any timestamps in a Mover blackboard, be aware that inconsistent use of float and double as the element type can lead to reading bad data. If you 'Set' a blackboard item using a double, you need to 'Get' it as a double as well.
