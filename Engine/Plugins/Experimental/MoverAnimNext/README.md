# Mover AnimNext Plugin

The Mover AnimNext Plugin adds Mover specific nodes to anim next. See Mover and AnimNext Plugin documentation for details.

**The Mover and Move AnimNext plugin are Experimental. Many features are incomplete or missing. APIs and data formats are subject to change at any time.**

