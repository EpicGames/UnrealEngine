Unreal Python API HTML generation using the Python Online Docs Commandlet
===========================================================

This commandlet can be run manually to generate the Python API HTML files locally or it can be set up on an automated server that periodically refreshes the docs online.


Install Unreal Engine and the Python plugin
---------------------------------------

Ensure the version of Unreal Engine that you want to be working with is installed and the PythonScriptPlugin is enabled. Python 3.7 now comes auto-installed with Unreal Engine and the doc generation commandlet will use that version so you don't need to install it yourself. All the other dependencies (such as the Sphinx Python doc generation tool) will automatically be downloaded and installed as needed when this commandlet is run.

Note that the auto-install of Sphinx adds about 50MB.


Ensure Sphinx template files are in place
---------------------------------------

[Should only need to do this step once and then it is set up for repeated builds.]

The Sphinx template files used for the doc generation are located here:
Engine\Plugins\Experimental\PythonScriptPlugin\SphinxDocs

The template files are in SphinxDocs\source\_templates and they are used by the PythonScriptPlugin when creating files used by Sphinx:

conf.py
This is the settings file for the Sphinx doc generation.
{{Version}} is replaced with the version of Unreal that is having docs generated.

index.rst
{{SectionList}} is replaced with a list of all the different section categories for quick access.
{{TableOfContents}} is replaced with several tables of contents for all the different categories of Unreal Python classes and placed at SphinxDocs\source\index.rst

Module.rst
{{Module}} is replaced with each exposed Python module and {{ModuleFunctions}} is replaced with all the module top level functions and data members.
Currently the only module is "unreal".

Class.rst
{{Class}} is replaced with the name of each specific Unreal Python class and each separate file is placed in SphinxDocs\source\class\ and named [ClassName].rst

You can add additional .rst files for additional documentation - see SphinxDocs\source\introduction.rst. Be sure to reference them in the SphinxDocs\source\_templates\index.rst file table of contents. Also try to put as little as possible in supplementary docs since they will not be automatically undergo localization like the Wordpress files for the rest of the Python docs.


You can find out more about Sphinx at http://www.sphinx-doc.org/en/master/

Most of the template files are written in reStructuredText format.

Here is a cheat sheet on Sphinx and reStructuredText - https://thomas-cokelaer.info/tutorials/sphinx/rest_syntax.html

You can add reStructuredText syntax highlighting to Notepad++:
https://github.com/steenhulthin/reStructuredText_NPP


Run the PythonOnlineDocs commandlet
---------------------------------------

The command-line arguments sent to the Unreal Engine Editor to run the PythonOnlineDocs commandlet looks like this for the Unreal Engine Editor with the standard Engine API documented:

-run=PythonOnlineDocs -NoShaderCompile -IncludeEngine -EnableAllPlugins

Enterprise version of command-line:

-run=PythonOnlineDocs -NoShaderCompile -IncludeEngine -IncludeEnterprise -EnableAllPlugins


The stubs for the Python API will be generated first and then all the files needed for Sphinx to generate HTML files. The HTML and associated files once generated will be about 200MB and will be placed in:
Engine\Plugins\Experimental\PythonScriptPlugin\SphinxDocs\build

The files located in PythonScriptPlugin\SphinxDocs\build are relative and can be used without a web server. Just drag the index.htm file to your browser of choice. Even the search works locally via a generated JavaScript search index file.

The files can be zipped up and distributed to be used locally or placed on a web server.

The objects.inv and the associated .doctrees folder is used by other Python API web sites for linking to this site. It is only needed when placed on an online web server and not needed when just using the files locally.


PythonOnlineDocs commandlet options
---------------------------------------

Here are the different command-line options relevant for the PythonOnlineDocs commandlet.

Filter options:
-InclueEngine      : Include the standard Engine Python API. Needed for all filter options.
-IncludeEnterprise : Include Enterprise Python API
-IncludeProject    : Include Python API from any loaded project

Other useful options:
-HTMLLog           : Output additional Python info, warnings and errors including feedback from running Sphinx (which has many warnings that can be safely ignored)
-NoShaderCompile   : Don't compile shaders (shaders aren't needed by the PythonOnlineDocs commandlet)
-EnableAllPlugins  : Include all the plugins known to the Unreal Engine editor
-ExceptPlugins=""  : Don't include specified plugins. Some plugins cause issues or aren't needed in the Python API docs.
-NoHTML            : Don't run Sphinx, potentially run Sphinx manually to build the HTML files see "Unreal Python API HTML generation manually using Sphinx"


Unreal Python API HTML generation manually using Sphinx
===========================================================

Use the same command-line as above with -NoHTML added:

-run=PythonOnlineDocs -NoShaderCompile -IncludeEngine -NoHTML -EnableAllPlugins


Ensure Python is installed
---------------------------------------

Download from https://www.python.org/downloads/

[Python 3.7.7 was used for testing and to make this guide since it matches the current needs of enterprise studios.]

You *may* need to set Python to use UTF-8 (if you have a code page set with "chcp 65001" for example). You can set it interactively on the command line:
set PYTHONIOENCODING=UTF-8

Or set it permanently in the environment variables.


Install Sphinx
---------------------------------------

[Should only need to do this step once and then it is set up for repeated builds.]

Install Sphinx using command prompt:
pip install -U sphinx

Confirm installation and version:
sphinx-build --version


Generate Python API HTML using Sphinx
---------------------------------------

[This step needs to be done for each refresh of the Unreal Python API whenever reflected engine bindings are changed.]

Using the command prompt (note: you must use the regular command prompt and not the Windows Powershell which may display colored text as invisible) go to the Python SphinxDocs folder:
Engine\Plugins\Experimental\PythonScriptPlugin\SphinxDocs

Run Sphinx to build the static Python API HTML:
sphinx-build -b html source/ build/

All the HTML files will be output to the directory:
Engine\Plugins\Experimental\PythonScriptPlugin\SphinxDocs\build

Sphinx looks at the docstrings in the unreal.py stub file during generation looking for reStructuredText markup. Many of the docstrings have things that Sphinx *thinks* are markup, though they are usually badly formed markup since people didn't know of the reStructuredText conventions when they wrote them. Such as: indentations without line gaps before and after, identifiers that end in an underscore_, and many other things. As of writing this, there are 159 warnings during the Sphinx generation process. Most can be safely ignored since they are accidental markup in the docstrings. None of them currently prevent the HTML generation.

