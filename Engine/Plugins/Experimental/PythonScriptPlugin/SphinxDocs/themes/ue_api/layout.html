{#
    ue_api/layout.html
    ~~~~~~~~~~~~~~~~~~~

    Sphinx layout template for the Unreal Engine API theme.
    
    Copyright Epic Games, Inc. All Rights Reserved.
#}
{%- extends "classic/layout.html" %}

{% if theme_collapsiblesidebar|tobool %}
{% set script_files = script_files + ['_static/sidebar.js'] %}
{% endif %}

{% block header %}
<script>window.__tracking_base = 'https://tracking.unrealengine.com';</script>
<script src="https://tracking.unrealengine.com/tracking.js" async></script>
<div id="head"></div>
<script>
$(document).ready(function () {
    $.ajax({
        url: "https://www.unrealengine.com/epic-games-header?useServerUrl=true&lang=en-US",
        context: document.body
    }).done(function (data) {
        $("#head").html(data);
    }).fail(function( jqXHR, textStatus, errorThrown ) {
	});
});

window._egNavAfterConfigSet = function() {
	_epicGamesNav.hideSignIn = true;
	_epicGamesNav.hideLocale = true;
};
</script>
{% endblock %}