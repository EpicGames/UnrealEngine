/*
 * sidebar.js
 * ~~~~~~~~~~
 *
 * This script makes the Sphinx sidebar collapsible.
 *
 * .sphinxsidebar contains .sphinxsidebarwrapper.  This script adds
 * in .sphixsidebar, after .sphinxsidebarwrapper, the #sidebarbutton
 * used to collapse and expand the sidebar.
 *
 * When the sidebar is collapsed the .sphinxsidebarwrapper is hidden
 * and the width of the sidebar and the margin-left of the document
 * are decreased. When the sidebar is expanded the opposite happens.
 * This script saves a per-browser/per-session cookie used to
 * remember the position of the sidebar among the pages.
 * Once the browser is closed the cookie is deleted and the position
 * reset to the default (expanded).
 *
 * :copyright: Copyright Epic Games, Inc. All Rights Reserved.
 *
 */

$(function() {
  {% if theme_rightsidebar|tobool %}
  {% set side = 'right' %}
  {% set opposite = 'left' %}
  {% set initial_label = '&raquo;' %}
  {% set expand_label = '«' %}
  {% set collapse_label = '»' %}
  {% else %}
  {% set side = 'left' %}
  {% set opposite = 'right' %}
  {% set initial_label = '&laquo;' %}
  {% set expand_label = '»' %}
  {% set collapse_label = '«' %}
  {% endif %}

  // global elements used by the functions.
  // the 'sidebarbutton' element is defined as global after its
  // creation, in the add_sidebar_button function
  var bodywrapper = $('.bodywrapper');
  var sidebar = $('.sphinxsidebar');
  var sidebarwrapper = $('.sphinxsidebarwrapper');

  // for some reason, the document has no sidebar; do not run into errors
  if (!sidebar.length) return;

  // original margin-left of the bodywrapper and width of the sidebar
  // with the sidebar expanded
  var bw_margin_expanded = bodywrapper.css('margin-{{side}}');
  var ssb_width_expanded = sidebar.width();

  // margin-left of the bodywrapper and width of the sidebar
  // with the sidebar collapsed
  var bw_margin_collapsed = '.8em';
  var ssb_width_collapsed = '.8em';

  // colors used by the current theme
  var dark_color = $('.related').css('background-color');
  var light_color = $('.document').css('background-color');

  function sidebar_is_collapsed() {
    return sidebarwrapper.is(':not(:visible)');
  }

  function toggle_sidebar() {
    if (sidebar_is_collapsed())
      expand_sidebar();
    else
      collapse_sidebar();
  }

  function collapse_sidebar() {
    sidebarwrapper.hide();
    sidebar.css('width', ssb_width_collapsed);
    bodywrapper.css('margin-{{side}}', bw_margin_collapsed);
    sidebarbutton.css({
        'margin-{{side}}': '0',
        'height': bodywrapper.height()
    });
    sidebarbutton.find('span').text('{{expand_label}}');
    sidebarbutton.attr('title', _('Expand sidebar'));
    document.cookie = 'sidebar=collapsed';
  }

  function expand_sidebar() {
    bodywrapper.css('margin-{{side}}', bw_margin_expanded);
    sidebar.css('width', ssb_width_expanded);
    sidebarwrapper.show();
    sidebarbutton.css({
        'margin-{{side}}': ssb_width_expanded-12,
        'height': bodywrapper.height()
    });
    sidebarbutton.find('span').text('{{collapse_label}}');
    sidebarbutton.attr('title', _('Collapse sidebar'));
    document.cookie = 'sidebar=expanded';
  }

  function add_sidebar_button() {
    sidebarwrapper.css({
        'float': '{{side}}',
        'margin-{{opposite}}': '0',
        'width': ssb_width_expanded - 28
    });
    // create the button
    sidebar.append(
        '<div id="sidebarbutton"><span>{{initial_label}}</span></div>'
    );
    var sidebarbutton = $('#sidebarbutton');
    light_color = sidebarbutton.css('background-color');
    // find the height of the viewport to center the '<<' in the page
    var viewport_height;
    if (window.innerHeight)
 	  viewport_height = window.innerHeight;
    else
	  viewport_height = $(window).height();
    sidebarbutton.find('span').css({
        'display': 'block',
        'margin-top': (viewport_height - sidebar.position().top - 20) / 2
    });

    sidebarbutton.click(toggle_sidebar);
    sidebarbutton.attr('title', _('Collapse sidebar'));
    sidebarbutton.css({
        'color': '#FFFFFF',
        'border-{{side}}': '1px solid ' + dark_color,
        'font-size': '1.2em',
        'cursor': 'pointer',
        'height': bodywrapper.height(),
        'padding-top': '1px',
        'margin-{{side}}': ssb_width_expanded - 12
    });

    sidebarbutton.hover(
      function () {
          $(this).css('background-color', dark_color);
      },
      function () {
          $(this).css('background-color', light_color);
      }
    );
  }

  function set_position_from_cookie() {
    if (!document.cookie)
      return;
    var items = document.cookie.split(';');
    for(var k=0; k<items.length; k++) {
      var key_val = items[k].split('=');
      var key = key_val[0].replace(/ /, "");  // strip leading spaces
      if (key == 'sidebar') {
        var value = key_val[1];
        if ((value == 'collapsed') && (!sidebar_is_collapsed()))
          collapse_sidebar();
        else if ((value == 'expanded') && (sidebar_is_collapsed()))
          expand_sidebar();
      }
    }
  }

  add_sidebar_button();
  var sidebarbutton = $('#sidebarbutton');
  set_position_from_cookie();
});
