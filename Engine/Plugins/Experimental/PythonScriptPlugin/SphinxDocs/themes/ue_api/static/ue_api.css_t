/*
 * Copyright Epic Games, Inc. All Rights Reserved.
 *
 * ue_api.css_t
 * ~~~~~~~~~~~~~
 *
 * Sphinx stylesheet -- Unreal Engine API theme.
 *
 */

@import url("basic.css");

/* -- page layout ----------------------------------------------------------- */

body {
    font-family: {{ theme_bodyfont }};
    font-size: 100%;
    background-color: {{ theme_footerbgcolor }};
    color: #000;
    margin: 0;
    padding: 0;
}

div.document {
    background-color: {{ theme_sidebarbgcolor }};
}

div.documentwrapper {
    float: left;
    width: 100%;
}

div.bodywrapper {
    margin: 0 0 0 {{ theme_sidebarwidth|todim }};
}

div.body {
    background-color: {{ theme_bgcolor }};
    color: {{ theme_textcolor }};
    padding: 0 20px 30px 20px;
}

{%- if theme_rightsidebar|tobool %}
div.bodywrapper {
    margin: 0 {{ theme_sidebarwidth|todim }} 0 0;
}
{%- endif %}

div.footer {
    color: {{ theme_footertextcolor }};
    width: 100%;
    padding: 9px 0 9px 0;
    text-align: center;
    font-size: 75%;
}

div.footer a {
    color: {{ theme_footertextcolor }};
    text-decoration: underline;
}

div.related {
    background-color: {{ theme_relbarbgcolor }};
    line-height: 30px;
    color: {{ theme_relbartextcolor }};
}

div.related a {
    color: {{ theme_relbarlinkcolor }};
}

div.sphinxsidebar {
    {%- if theme_stickysidebar|tobool %}
    top: 80px;
    bottom: 0;
    margin: 0;
    position: fixed;
    overflow: auto;
    height: auto;
    {%- endif %}
    {%- if theme_rightsidebar|tobool %}
    float: right;
    {%- if theme_stickysidebar|tobool %}
    right: 0;
    {%- endif %}
    {%- endif %}
}

{%- if theme_stickysidebar|tobool %}
/* this is nice, but it it leads to hidden headings when jumping
   to an anchor */
/*
div.related {
    position: fixed;
}

div.documentwrapper {
    margin-top: 30px;
}
*/
{%- endif %}

div.sphinxsidebar h3 {
    font-family: {{ theme_headfont }};
    color: {{ theme_sidebartextcolor }};
    font-size: 1.4em;
    font-weight: normal;
    margin: 0;
    padding: 0;
}

div.sphinxsidebar h3 a {
    color: {{ theme_sidebartextcolor }};
}

div.sphinxsidebar h4 {
    font-family: {{ theme_headfont }};
    color: {{ theme_sidebartextcolor }};
    font-size: 1.3em;
    font-weight: normal;
    margin: 5px 0 0 0;
    padding: 0;
}

div.sphinxsidebar p {
    color: {{ theme_sidebartextcolor }};
}

div.sphinxsidebar p.topless {
    margin: 5px 10px 10px 10px;
}

div.sphinxsidebar ul {
    margin: 10px;
    padding: 0;
    color: {{ theme_sidebartextcolor }};
}

div.sphinxsidebar a {
    color: {{ theme_sidebarlinkcolor }};
}

div.sphinxsidebar input {
    border: 1px solid {{ theme_sidebarlinkcolor }};
    font-family: sans-serif;
    font-size: 1em;
}

{% if theme_collapsiblesidebar|tobool %}
/* for collapsible sidebar */
div#sidebarbutton {
    background-color: {{ theme_sidebarbtncolor }};
}
{% endif %}

/* -- hyperlink styles ------------------------------------------------------ */

a {
    color: {{ theme_linkcolor }};
    text-decoration: none;
}

a:visited {
    color: {{ theme_visitedlinkcolor }};
    text-decoration: none;
}

a:hover {
    text-decoration: underline;
}

{% if theme_externalrefs|tobool %}
a.external {
   text-decoration: none;
   border-bottom: 1px dashed {{ theme_linkcolor }};
}

a.external:hover {
   text-decoration: none;
   border-bottom: none;
}

a.external:visited {
    text-decoration: none;
    border-bottom: 1px dashed {{ theme_visitedlinkcolor }};
}
{% endif %}

/* -- body styles ----------------------------------------------------------- */

div.body h1,
div.body h2,
div.body h3,
div.body h4,
div.body h5,
div.body h6 {
    font-family: {{ theme_headfont }};
    background-color: {{ theme_headbgcolor }};
    font-weight: normal;
    color: {{ theme_headtextcolor }};
    border-bottom: 1px solid #ccc;
    margin: 20px -20px 10px -20px;
    padding: 3px 0 3px 10px;
}

div.body h1 { margin-top: 0; font-size: 200%; }
div.body h2 { font-size: 160%; }
div.body h3 { font-size: 140%; }
div.body h4 { font-size: 120%; }
div.body h5 { font-size: 110%; }
div.body h6 { font-size: 100%; }

a.headerlink {
    color: {{ theme_headlinkcolor }};
    font-size: 0.8em;
    padding: 0 4px 0 4px;
    text-decoration: none;
}

a.headerlink:hover {
    background-color: {{ theme_headlinkcolor }};
    color: white;
}

div.body p, div.body dd, div.body li, div.body blockquote {
    text-align: left;
    line-height: 130%;
}

div.admonition p {
    margin-bottom: 5px;
}

div.admonition pre {
    margin-bottom: 5px;
}

div.admonition ul, div.admonition ol {
    margin-bottom: 5px;
}

div.note {
    background-color: #eee;
    border: 1px solid #ccc;
}

div.seealso {
    background-color: #ffc;
    border: 1px solid #ff6;
}

div.topic {
    background-color: #eee;
}

div.warning {
    background-color: #ffe4e4;
    border: 1px solid #f66;
}

p.admonition-title:after {
    content: ":";
}

pre {
    padding: 5px;
    background-color: {{ theme_codebgcolor }};
    color: {{ theme_codetextcolor }};
    line-height: 120%;
    border: 1px solid #ac9;
    border-left: none;
    border-right: none;
}

code {
    background-color: rgba(0,0,128,0.08);
    padding: 0 1px 0 1px;
    font-size: 1.1em;
}

code.descname {
    font-size: 1.4em;
}

dd ul code {
    font-size: 1.3em;
    font-weight: bold;
    background-color: transparent;
}

th {
    background-color: {{ theme_th_bgcolor }};
}

.warning code {
    background: #efc2c2;
}

.note code {
    background: #d6d6d6;
}

.viewcode-back {
    font-family: {{ theme_bodyfont }};
}

div.viewcode-block:target {
    background-color: #f4debf;
    border-top: 1px solid #ac9;
    border-bottom: 1px solid #ac9;
}

div.code-block-caption {
    color: #efefef;
    background-color: #1c4e63;
}


table.docutils td, table.docutils th {
    padding: 1px 8px 1px 5px;
    border-top: 1px solid #cce;
    border-left: 1px solid #cce;
    border-right: 1px solid #cce;
    border-bottom: 1px solid #cce;
}

dl {
    margin-bottom: 1.5em;
}

dl.method {
    border-top: 3px solid #cce;
    padding-top: .33em;
}

dl.classmethod {
    border-top: 3px solid #cec;
    padding-top: .33em;
}

dl.attribute {
    border-top: 3px solid #fca;
    padding-top: .33em;
}

span.caption-text {
    font-weight: bold;
}
