# Copyright Epic Games, Inc. All Rights Reserved.

[theme]
inherit = classic
stylesheet = ue_api.css
pygments_style = sphinx

[options]
rightsidebar = false
stickysidebar = false
collapsiblesidebar = false
externalrefs = false

footerbgcolor    = #11303d
footertextcolor  = #ffffff
sidebarbgcolor   = #1c4e63
sidebarbtncolor  = #3c6e83
sidebartextcolor = #ffffff
sidebarlinkcolor = #98dbcc
relbarbgcolor    = #133f52
relbartextcolor  = #ffffff
relbarlinkcolor  = #ffffff
bgcolor          = #ffffff
textcolor        = #000000
headbgcolor      = #f2f2f2
headtextcolor    = #20435c
headlinkcolor    = #c60f0f
linkcolor        = #355f7c
visitedlinkcolor = #355f7c
codebgcolor      = #eeffcc
codetextcolor    = #333333
th_bgcolor       = #f0f0f0

bodyfont = sans-serif
headfont = 'Trebuchet MS', sans-serif
