Note: Plugins in this directory are unsupported prototypes.  They will be promoted out of this folder if/when they are production ready.

Data or code compatibility is not guaranteed.  Use these at your own risk!
