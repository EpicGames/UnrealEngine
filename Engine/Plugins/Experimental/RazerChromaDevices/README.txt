# A note about Razer Chroma PDB files

If you would like to compile with the RazerChroma PDB file for better
debugging symbols, then you should download them from the Razer CChromaEditor
github : https://github.com/razerofficial/CChromaEditor

You should put the .pdb files in the following directories to build successfully:

Win64: Engine\Plugins\Experimental\RazerChromaDevices\Binaries\ThirdParty\Win64\CChromaEditorLibrary64.pdb
Win32: Engine\Plugins\Experimental\RazerChromaDevices\Binaries\ThirdParty\Win32\CChromaEditorLibrary.pdb