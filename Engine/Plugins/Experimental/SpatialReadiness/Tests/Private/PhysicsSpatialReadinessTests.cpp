// Copyright Epic Games, Inc. All Rights Reserved.
#include "TestHarness.h"
#include "CoreMinimal.h"
#include "PBDRigidsSolver.h"
#include "ChaosSolversModule.h"
#include "Physics/Experimental/PhysScene_Chaos.h"
#include "SpatialReadinessAPI.h"
#include "SpatialReadinessSimCallback.h"
#include "Engine/EngineTypes.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"
#include "Physics/PhysicsFiltering.h"

using Chaos::FSingleParticlePhysicsProxy;
using Chaos::FPBDRigidParticleHandle;
using Chaos::FVec3;
using Chaos::FReal;
using Chaos::FRigidBodyHandle_External;
using Chaos::TBox;
using Chaos::FImplicitObjectPtr;
using Chaos::EObjectStateType;
using Chaos::FPBDRigidsSolver;
using Chaos::FPhysicsThreadContextScope;
using Chaos::EThreadingModeTemp;

namespace
{
	const FBox UnitBounds(FVector(-.5), FVector(.5));

	void AdvanceAndWait(Chaos::FPBDRigidsSolver* InSolver, float DeltaTime = 1)
	{
		if (InSolver)
		{
			InSolver->AdvanceAndDispatch_External(DeltaTime);
			InSolver->WaitOnPendingTasks_External();
		}
	}

	FSingleParticlePhysicsProxy* MakeDynamicBox(FChaosScene& Scene, const FBox& Bounds, const bool bGravityEnabled=true)
	{
		// Create a box implicit geometry from the same bounds as the unready volume
		const FVec3 BoxCenter = (Bounds.Min + Bounds.Max) * .5f;
		const FVec3 BoxHalfExtent = Bounds.Max - Bounds.Min;
		FImplicitObjectPtr BoxGeom = MakeImplicitObjectPtr<TBox<FReal, 3>>(-BoxHalfExtent, BoxHalfExtent);

		// Create a new static particle to represent the volume
		FActorCreationParams Params;
		Params.bSimulatePhysics = true;
		Params.bStatic = false;
		Params.InitialTM = FTransform(FQuat::Identity, BoxCenter);
		Params.Scene = &Scene;
		FSingleParticlePhysicsProxy* ParticleProxy = nullptr;
		FChaosEngineInterface::CreateActor(Params, ParticleProxy);
		FRigidBodyHandle_External& ParticleHandle = ParticleProxy->GetGameThreadAPI();
		REQUIRE(ParticleProxy);

		// Create collision filter data for the particle
		Chaos::Filter::FShapeFilterBuilder Builder;
		Builder.SetCollisionChannelIndex(ECollisionChannel::ECC_WorldDynamic);
		Builder.SetBlockChannelMask(TNumericLimits<uint64>::Max());
		Builder.SetFilterFlags(Chaos::EFilterFlags::CCD);
		const Chaos::Filter::FShapeFilterData ShapeFilterData = Builder.Build();
		

		// Make the geometry
		ParticleHandle.SetGeometry(BoxGeom);
		ParticleHandle.SetShapeSimCollisionEnabled(0, true);
		ParticleHandle.SetShapeQueryCollisionEnabled(0, false);
		ParticleHandle.SetShapeFilterData(0, ShapeFilterData);
		ParticleHandle.SetGravityEnabled(bGravityEnabled);
		ParticleHandle.SetObjectState(EObjectStateType::Dynamic, false, true);
#if CHAOS_DEBUG_NAME
		ParticleHandle.SetDebugName(MakeShared<FString>(TEXT("Dynamic Box")));
#endif

		// Add the new particle to the scene
		TArray<FPhysicsActorHandle> Actors = { ParticleProxy };
		Scene.AddActorsToScene_AssumesLocked(Actors);

		// Return the particle proxy
		return ParticleProxy;
	}

	class FTestSpatialReadinessSimCallback : public FSpatialReadinessSimCallback
	{
	public:
		FTestSpatialReadinessSimCallback(FPhysScene_Chaos& InScene) : FSpatialReadinessSimCallback(InScene) { }

		const TSet<FSingleParticlePhysicsProxy*>& GetUnreadyVolumeParticles_PT() const
		{
			return UnreadyVolumeParticles_PT;
		}

		const TSet<FPBDRigidParticleHandle*> GetUnreadyRigidParticles_PT() const
		{
			TSet<FPBDRigidParticleHandle*> RigidParticles;
			ForEachUnreadyRigidParticle_PT([&](FPBDRigidParticleHandle* RigidParticle) { RigidParticles.Add(RigidParticle); return true; });
			return RigidParticles;
		}

		TFunction<void()> OnPreSimulate_Callback = [](){};

	protected:

		virtual void OnPreSimulate_Internal() override
		{
			FSpatialReadinessSimCallback::OnPreSimulate_Internal();
			OnPreSimulate_Callback();
		}
	};
}

TEST_CASE("Sim Callback", "[physics spatial readiness]")
{
	// Create a solver in the solvers module
	FPhysScene_Chaos Scene;
	Chaos::FPBDRigidsSolver* Solver = Scene.GetSolver();
	Solver->SetThreadingMode_External(EThreadingModeTemp::TaskGraph);

	// Create a test SpatialReadinessSimCallback
	FTestSpatialReadinessSimCallback* SimCallback = Solver->CreateAndRegisterSimCallbackObject_External<FTestSpatialReadinessSimCallback>(Scene);
	AdvanceAndWait(Solver);

	// Create an API object which is hooked up to the SimCallback's functions
	FSpatialReadinessAPI SpatialReadiness(
		static_cast<FSpatialReadinessSimCallback*>(SimCallback),
		&FTestSpatialReadinessSimCallback::AddUnreadyVolume_GT,
		&FTestSpatialReadinessSimCallback::RemoveUnreadyVolume_GT);

	// Make a physics thread scope to avoid any thread context checks
	FPhysicsThreadContextScope PTScope(true);

	SECTION("Spawn dynamic particle overlapping unready volume, no gravity")
	{
		// Make a volume
		FSpatialReadinessVolume Volume = SpatialReadiness.CreateVolume(UnitBounds, TEXT("Test Volume"));
		AdvanceAndWait(Solver);
		REQUIRE(SimCallback->GetUnreadyRigidParticles_PT().Num() == 0);

		// Make a particle which should overlap that volume
		FSingleParticlePhysicsProxy* BoxProxy = MakeDynamicBox(Scene, UnitBounds, false);
		AdvanceAndWait(Solver);

		// Make sure the particle is frozen immediately by the registry callback
		REQUIRE(SimCallback->GetUnreadyRigidParticles_PT().Num() == 1);

		// Advance one more tick to detect the midphase
		AdvanceAndWait(Solver);
		REQUIRE(SimCallback->GetUnreadyRigidParticles_PT().Num() == 1);
	}

	SECTION("Spawn dynamic particle overlapping unready volume, with gravity")
	{
		// Make a volume
		FSpatialReadinessVolume Volume = SpatialReadiness.CreateVolume(UnitBounds, TEXT("Test Volume"));
		AdvanceAndWait(Solver);
		REQUIRE(SimCallback->GetUnreadyRigidParticles_PT().Num() == 0);

		// Make a particle which should overlap that volume
		FSingleParticlePhysicsProxy* BoxProxy = MakeDynamicBox(Scene, UnitBounds, true);
		AdvanceAndWait(Solver);
		REQUIRE(SimCallback->GetUnreadyRigidParticles_PT().Num() == 1);

		// Record the initial position
		const FVec3 FrozenX0 = BoxProxy->GetPhysicsThreadAPI()->X();

		// Advance ten more ticks to make sure it never starts falling
		for (int32 Iteration = 0; Iteration < 10; ++Iteration)
		{
			AdvanceAndWait(Solver);
			REQUIRE(SimCallback->GetUnreadyRigidParticles_PT().Num() == 1);
			const FVec3 FrozenX1 = BoxProxy->GetPhysicsThreadAPI()->X();
			REQUIRE(FrozenX0 == FrozenX1);
		}
	}

	SECTION("Dynamic particle with gravity stops falling when it hits an unready volume")
	{
		// Make a volume
		FBox UnreadyBox(FVector(-1000, -1000, -10000), FVector(1000, 1000, 0));
		FSpatialReadinessVolume Volume = SpatialReadiness.CreateVolume(UnreadyBox, TEXT("Test Volume"));
		AdvanceAndWait(Solver);
		REQUIRE(SimCallback->GetUnreadyRigidParticles_PT().Num() == 0);

		// Make a particle which is just above the unready volume
		FBox FallingBox(FVector(-1, -1, 2), FVector(1, 1, 3));
		FSingleParticlePhysicsProxy* BoxProxy = MakeDynamicBox(Scene, FallingBox, true);
		AdvanceAndWait(Solver);

		// Advance one more tick to detect the midphase and record the position of the particle
		AdvanceAndWait(Solver);
		REQUIRE(SimCallback->GetUnreadyRigidParticles_PT().Num() == 1);
		const FVec3 FrozenX0 = BoxProxy->GetPhysicsThreadAPI()->X();

		// Advance again and make sure the box didn't move. Do it 10 times for good measure
		for (int32 Iteration = 0; Iteration < 10; ++Iteration)
		{
			AdvanceAndWait(Solver);
			const FVec3 FrozenX1 = BoxProxy->GetPhysicsThreadAPI()->X();
			REQUIRE(FrozenX0 == FrozenX1);
		}

		// Mark the volume as "ready" and advance again - the particle
		// should be removed from the list of unready particles after 2 ticks
		// because spatial readiness always keeps particles unready for one extra tick.
		Volume.MarkReady();
		AdvanceAndWait(Solver);
		REQUIRE(SimCallback->GetUnreadyRigidParticles_PT().Num() == 1);
		AdvanceAndWait(Solver);
		REQUIRE(SimCallback->GetUnreadyRigidParticles_PT().Num() == 1);
		AdvanceAndWait(Solver);
		REQUIRE(SimCallback->GetUnreadyRigidParticles_PT().Num() == 0);

		// Advancing one more frame, we should see the particle start to fall
		// again.
		AdvanceAndWait(Solver);
		const FVec3 FrozenX2 = BoxProxy->GetPhysicsThreadAPI()->X();
		REQUIRE(FrozenX2.Z < FrozenX0.Z);
	}

	SECTION("Force added to frozen dynamic particle does nothing")
	{
		// Make a volume
		FSpatialReadinessVolume Volume = SpatialReadiness.CreateVolume(UnitBounds, TEXT("Test Volume"));
		AdvanceAndWait(Solver);
		REQUIRE(SimCallback->GetUnreadyRigidParticles_PT().Num() == 0);

		// Make a particle which should overlap that volume
		FSingleParticlePhysicsProxy* BoxProxy = MakeDynamicBox(Scene, UnitBounds, true);
		AdvanceAndWait(Solver);

		// Get the position of the box
		const FVec3 FrozenX0 = BoxProxy->GetPhysicsThreadAPI()->X();

		// Add a force to the box in pre-simulate
		SimCallback->OnPreSimulate_Callback = [BoxProxy]()
		{
			BoxProxy->GetPhysicsThreadAPI()->AddForce(FVec3(100,0,0));
		};

		// Advance again and make sure the box didn't move. Do it 10 times for good measure
		for (int32 Iteration = 0; Iteration < 10; ++Iteration)
		{
			AdvanceAndWait(Solver);
			const FVec3 FrozenX1 = BoxProxy->GetPhysicsThreadAPI()->X();
			REQUIRE(FrozenX0 == FrozenX1);
		}
	}

	SECTION("Game thread query for readiness")
	{
		// Make a volume
		FBox UnreadyBox(FVector(-100), FVector(100));
		FSpatialReadinessVolume Volume = SpatialReadiness.CreateVolume(UnreadyBox, TEXT("Test Volume"));

		// Make temp vars
		TArray<int32> VolumeIndices;
		bool bIsReady;

		// Do a query which should intersect the unready area
		bIsReady = SimCallback->QueryReadiness_GT(FBox(FVector(0), FVector(100)), VolumeIndices);
		REQUIRE(!bIsReady);
		REQUIRE(VolumeIndices.Num() == 1);
		REQUIRE(VolumeIndices[0] == 0);

		// Do a query which should not intersect the unready area
		bIsReady = SimCallback->QueryReadiness_GT(FBox(FVector(200), FVector(300)), VolumeIndices);
		REQUIRE(bIsReady);
		REQUIRE(VolumeIndices.Num() == 0);
	}

	SECTION("Add and remove volumes in different orders")
	{
		FBox UnreadyBox(FVector(-1), FVector(1));
		TArray<FSpatialReadinessVolume> Volumes;

		// add/remove
		Volumes.Emplace(SpatialReadiness.CreateVolume(UnreadyBox, TEXT("Test Volume")));
		AdvanceAndWait(Solver);
		Volumes.RemoveAt(0);
		AdvanceAndWait(Solver);

		// add/add/remove
		Volumes.Emplace(SpatialReadiness.CreateVolume(UnreadyBox, TEXT("Test Volume")));
		AdvanceAndWait(Solver);
		Volumes.Emplace(SpatialReadiness.CreateVolume(UnreadyBox, TEXT("Test Volume")));
		AdvanceAndWait(Solver);
		Volumes.RemoveAt(0);
		AdvanceAndWait(Solver);
	}

	SECTION("Add and remove many random volumes")
	{
		const int32 NumVolumeActions = 200;
		FRandomStream RandStream(42);

		TArray<FSpatialReadinessVolume> Volumes;

		for (int32 Index = 0; Index < NumVolumeActions; ++Index)
		{
			const int32 ActionIndex = RandStream.RandRange(0, 1);
			switch (ActionIndex)
			{
				// Add a volume
				case 0:
				{
					const FVector BoxMin = FVector(
						RandStream.FRandRange(-2000, 1000),
						RandStream.FRandRange(-2000, 1000),
						RandStream.FRandRange(-2000, 1000));
					const FVector BoxMax = BoxMin + FVector(
						RandStream.FRandRange(0, 1000),
						RandStream.FRandRange(0, 1000),
						RandStream.FRandRange(0, 1000));
					const FBox UnreadyBox(BoxMin, BoxMax);
					Volumes.Emplace(SpatialReadiness.CreateVolume(UnreadyBox, TEXT("Test Volume")));
					break;
				}

				// Remove a volume
				case 1:
				{
					if (Volumes.Num() == 0) { break; }
					const int32 VolumeIndex = RandStream.RandRange(0, Volumes.Num() - 1);
					Volumes.RemoveAt(VolumeIndex);
					break;
				}
			}

			AdvanceAndWait(Solver);
		}
	}
}
