// Copyright Epic Games, Inc. All Rights Reserved.
#include "TestHarness.h"
#include "SpatialReadinessAPI.h"

namespace
{
	// Make some test data for populating handles
	static const FBox TestBox(FVector(-.5f), FVector(.5f));
	static const FString TestDesc(TEXT("test volume"));
}

TEST_CASE("Spatial readiness api", "[spatial readiness]")
{
	// Make some lambdas that we can use to create spatial readiness
	// api objects that do nothing.
	struct FVolume { FBox Bounds; FString Desc; };
	TSparseArray<FVolume> Volumes;
	const auto Stub_AddVolume = [&Volumes](const FBox& Bounds, const FString& Desc) -> int32
	{
		int32 Index = 0;
		return Volumes.EmplaceAtLowestFreeIndex(Index, Bounds, Desc);
	};

	const auto Stub_RemoveVolume = [&Volumes](int32 Index) -> void
	{
		Volumes.RemoveAt(Index);
	};

	// Make a spatial readiness api
	TUniquePtr<FSpatialReadinessAPI> SpatialReadiness = MakeUnique<FSpatialReadinessAPI>(Stub_AddVolume, Stub_RemoveVolume);

	SECTION("Volume is initially unready")
	{
		FSpatialReadinessVolume Volume = SpatialReadiness->CreateVolume(TestBox, TestDesc);
		REQUIRE(Volume.IsReady() == false);
	}

	SECTION("Volume is removed when volume is marked ready")
	{
		FSpatialReadinessVolume Volume = SpatialReadiness->CreateVolume(TestBox, TestDesc);
		REQUIRE(Volumes.Num() == 1);
		Volume.MarkReady();
		REQUIRE(Volumes.Num() == 0);
	}

	SECTION("Volume is added when volume is marked unready")
	{
		FSpatialReadinessVolume Volume = SpatialReadiness->CreateVolume(TestBox, TestDesc);
		Volume.MarkReady();
		Volume.MarkUnready();
		REQUIRE(Volumes.Num() == 1);
	}

	SECTION("Volume is marked ready twice")
	{
		FSpatialReadinessVolume Volume = SpatialReadiness->CreateVolume(TestBox, TestDesc);
		Volume.MarkReady();
		Volume.MarkReady();
	}

	SECTION("Volume is marked unready when already unready")
	{
		FSpatialReadinessVolume Volume = SpatialReadiness->CreateVolume(TestBox, TestDesc);
		Volume.MarkUnready();
	}

	SECTION("Volume is removed when volume goes out of scope")
	{
		{
			FSpatialReadinessVolume Volume = SpatialReadiness->CreateVolume(TestBox, TestDesc);
			REQUIRE(Volumes.Num() == 1);
		}

		REQUIRE(Volumes.Num() == 0);
	}

	SECTION("Volume becomes valid when api goes out of scope")
	{
		// Make a volume volume and require that it's valid
		FSpatialReadinessVolume Volume = SpatialReadiness->CreateVolume(TestBox, TestDesc);
		REQUIRE(Volume.IsValid());
		REQUIRE(Volumes.Num() == 1);

		// Delete the readiness api and ensure that the volume is now invalid
		SpatialReadiness.Reset();
		REQUIRE(!Volume.IsValid());
	}

	/*
	SECTION("Ensure is fired when api callback causes its own deletion")
	{
		struct FSelfDeletingSpatialReadinessProvider
		{
			FSelfDeletingSpatialReadinessProvider() : SpatialReadiness(MakeUnique<FSpatialReadinessAPI>(this, &FSelfDeletingSpatialReadinessProvider::AddUnreadyVolume, &FSelfDeletingSpatialReadinessProvider::RemoveUnreadyVolume))
			{ }

			int32 AddUnreadyVolume(const FBox&, const FString&) { return 0; }
			void RemoveUnreadyVolume(const int32)
			{
				// This is a terrible thing to do, but the idea is to 
				SpatialReadiness.Reset();
			}

			TUniquePtr<FSpatialReadinessAPI> SpatialReadiness;
		};

		// Create an instance of the provider, generate a handle, and mark
		// it ready to delete it. This should trigger a warning.
		FSelfDeletingSpatialReadinessProvider Provider;
		FSpatialReadinessVolume Handle = Provider.SpatialReadiness->CreateHandle(TestBox, TestDesc);
		Handle.MarkReady();
	}
	*/
}

TEST_CASE("Spatial readiness provider class", "[spatial readiness]")
{
	// Create a spatial readiness provider class to mimic ones that might exist
	struct FTestSpatialReadinessProvider
	{
		FTestSpatialReadinessProvider() : SpatialReadiness(this, &FTestSpatialReadinessProvider::AddUnreadyVolume, &FTestSpatialReadinessProvider::RemoveUnreadyVolume)
		{ }

		int32 AddUnreadyVolume(const FBox&, const FString&) { ++VolumesAdded; return 0; }

		void RemoveUnreadyVolume(const int32) { ++VolumesRemoved; }

		FSpatialReadinessAPI SpatialReadiness;

		int32 VolumesAdded = 0;
		int32 VolumesRemoved = 0;
	};

	// Make an instance of the provider
	FTestSpatialReadinessProvider Provider;

	SECTION("Instantiate spatial readiness api with member function bindings")
	{
		{
			// This should trigger calls to add unready volume
			FSpatialReadinessVolume Volume = Provider.SpatialReadiness.CreateVolume(TestBox, TestDesc);
			REQUIRE(Volume.IsValid());
			REQUIRE(Volume.IsReady() == false);
			REQUIRE(Provider.VolumesAdded == 1);
			REQUIRE(Provider.VolumesRemoved == 0);
		}

		// The volume going out of scope should cause the volume to be removed
		REQUIRE(Provider.VolumesAdded == 1);
		REQUIRE(Provider.VolumesRemoved == 1);
	}
}