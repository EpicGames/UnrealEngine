// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class SpatialReadinessTests : TestModuleRules
{
	static SpatialReadinessTests()
	{
		TestMetadata = new Metadata()
		{
			TestName = "SpatialReadinessTests",
			TestShortName = "SpatialReadinessTests",
		};
	}

	public SpatialReadinessTests(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"ApplicationCore",
			"Core",
			"SpatialReadiness",
			"Chaos",
		});
	}
}