// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedPlatforms(UnrealPlatformClass.All)]
public class SpatialReadinessTestsTarget : TestTargetRules
{
	public SpatialReadinessTestsTarget(TargetInfo Target) : base(Target)
	{
		bUsePlatformFileStub = true;
		bMockEngineDefaults = true;
		TestTargetRules.bTestsRequireEngine = true;
	}
}
