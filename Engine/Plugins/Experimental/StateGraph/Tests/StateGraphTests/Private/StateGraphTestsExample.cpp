// Copyright Epic Games, Inc. All Rights Reserved.

#include "Containers/Ticker.h"
#include "Misc/ConfigCacheIni.h"
#include "StateGraph.h"
#include "TestHarness.h"

namespace UE::StateGraphTests::Example
{

// This example shows many of the features provided by the StateGraph module. See class descriptions below for how nodes relate to each other within the state graph.
// Running this produces the following debug output at the end:
//
// [UE::FStateGraph::Run] [Example] Completed (Duration=0.001272)
// [UE::FStateGraph::LogDebugInfo] [Example] Status=Completed Nodes=11 Duration=0.001272
// [UE::FStateGraph::LogDebugInfo] [Example.Base.NodeA] Status=Completed Duration=0.000023 Dependencies(None)
// [UE::FStateGraph::LogDebugInfo] [Example.Base.NodeB] Status=Completed Duration=0.000258 Dependencies(None)
// [UE::FStateGraph::LogDebugInfo] [Example.Base.NodeC] Status=Completed Duration=0.000206 Dependencies(Completed=Base.NodeA,Base.NodeB)
// [UE::FStateGraph::LogDebugInfo] [Example.Base.NodeD] Status=Completed Duration=0.000013 Dependencies(Completed=Base.NodeC,Derived.NodeF)
// [UE::FStateGraph::LogDebugInfo] [Example.Base.NodeE] Status=Completed Duration=0.000190 Dependencies(Completed=Base.NodeC)
// [UE::FStateGraph::LogDebugInfo] [Example.Derived.NodeF] Status=Completed Duration=0.000556 Dependencies(Completed=Base.NodeC)
// [UE::FStateGraph::LogDebugInfo] [Example.Derived.NodeG] Status=Completed Duration=0.000056 Dependencies(None)
// [UE::FStateGraph::LogDebugInfo] [Example.Derived.NodeH] Status=Completed Duration=0.000370 Dependencies(Completed=Derived.NodeG)
// [UE::FStateGraph::LogDebugInfo] [Example.Module.NodeA] Status=Completed Duration=0.000014 Dependencies(None)
// [UE::FStateGraph::LogDebugInfo] [Example.Module.NodeB] Status=Completed Duration=0.000321 Dependencies(Completed=Module.NodeA)
// [UE::FStateGraph::LogDebugInfo] [Example.Module.NodeC] Status=Completed Duration=0.000015 Dependencies(Completed=Module.NodeB)
//
// To see the graph work through all nodes, build and run: StateGraphTests.exe [Example] --log --extra-args -LogCmds="LogStateGraph All" -AbsLog=D:\StateGraph.log


// Use custom class nodes like this to manage functions and data for a single node, as opposed to the single function nodes seen below.
// Instances of this are added to the state graph in the classes below.
class FExampleClassNode : public FStateGraphNode
{
public:

	FExampleClassNode(FName InName) :
		FStateGraphNode(InName)
	{}

	float Input = 0.1f;
	float Output = 0;

protected:

	virtual void Start()
	{
		// Code to setup async task...
		FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateSP(this, &FExampleClassNode::RunAsync), Input);
	}

	bool RunAsync(float DeltaTime)
	{
		if (++RunState < CustomConfig)
		{
			return true;
		}

		FinishAsync();
		return false;
	}

	void FinishAsync()
	{
		// Code to process result of async task...
		Output = 0.2f;
		Complete();
	}

	virtual void UpdateConfig()
	{
		// Override to read custom config for the node.
		FStateGraphNode::UpdateConfig();
		GConfig->GetInt(*GetConfigSectionName(), TEXT("CustomConfig"), CustomConfig, GEngineIni);
	}

	int32 RunState = 0;
	int32 CustomConfig = 3;
};

using FExampleClassNodePtr = TSharedPtr<FExampleClassNode, ESPMode::ThreadSafe>;


// Helper function to simulate async tasks in simple function nodes below.
void ExampleAsync(float Delay, const FStateGraphNodeFunctionComplete& Complete)
{
	FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda([Complete](float DeltaTime) {
		Complete();
		return false;
	}), Delay);
}


// FExampleBase - Base class of some function nodes that perform some work.
class FExampleBase : public TSharedFromThis<FExampleBase>
{
public:

	FExampleBase(const FStateGraphRef& InStateGraph) :
		StateGraph(InStateGraph)
	{}

	virtual ~FExampleBase() = default;

	virtual void SetupStateGraph()
	{
		// CreateNode supports any function type that can be used to create delegates with: lambdas, members, static, raw, etc.

		// Since NodeA and NodeB have no dependencies, they run first concurrently.
		StateGraph->CreateNode("Base.NodeA", this, [](FStateGraph& StateGraph, FStateGraphNodeFunctionComplete Complete) { Complete(); });
		StateGraph->CreateNode("Base.NodeB", this, [](FStateGraph& StateGraph, FStateGraphNodeFunctionComplete Complete) { ExampleAsync(0.1f, Complete); });

		// CreateNode also supports custom node types by explicitly specifying the type and passing arguments to the constructor.
		FExampleClassNodePtr NodeC = StateGraph->CreateNode<FExampleClassNode>("Base.NodeC");
		// Add dependencies so NodeC isn't run until both NodeA and NodeB complete.
		NodeC->Dependencies.Append({ "Base.NodeA", "Base.NodeB" });
		NodeC->Input = 0.1f;

		// Both NodeD and NodeE will run concurrently once NodeC is complete.
		StateGraph->CreateNode("Base.NodeD", &FExampleBase::NodeD)
			->Dependencies.Add("Base.NodeC");
		StateGraph->CreateNode("Base.NodeE", this, &FExampleBase::NodeE)
			->Dependencies.Add("Base.NodeC");
	}

protected:

	static void NodeD(FStateGraph& InStateGraph, FStateGraphNodeFunctionComplete Complete)
	{
		Complete();
	}

	void NodeE(FStateGraph& InStateGraph, FStateGraphNodeFunctionComplete Complete)
	{
		FExampleClassNodePtr NodeC = StateGraph->GetNode<FExampleClassNode>("Base.NodeC");
		check(NodeC.IsValid());
		// Use NodeC Output within this node...
		ExampleAsync(NodeC->Output, Complete);
	}

	FStateGraphRef StateGraph;
};


// FExampleModule - Class representing a module that shares no nodes with FExampleBase or FExampleDerived, but is used by FExampleDerived to complete some work.
DECLARE_DELEGATE_OneParam(FExampleModuleComplete, int32 /* Output */);
class FExampleModule : public TSharedFromThis<FExampleModule>
{
public:
	FExampleModule(const FStateGraphRef& InStateGraph) :
		StateGraph(InStateGraph)
	{}

	void Run(const FExampleModuleComplete& ModuleComplete)
	{
		// Run nodes in sequence to produce some output that the caller needs, using the same state graph which may be running unrelated nodes.
		StateGraph->CreateNode("Module.NodeA", this, [](FStateGraph& StateGraph, FStateGraphNodeFunctionComplete Complete) { Complete(); })
			->Next("Module.NodeB", this, &FExampleModule::NodeB)
			->Next("Module.NodeC", this, [ModuleComplete](FStateGraph& StateGraph, FStateGraphNodeFunctionComplete Complete) { Complete(); ModuleComplete.ExecuteIfBound(42); });
	}

protected:

	void NodeB(FStateGraph& InStateGraph, FStateGraphNodeFunctionComplete Complete)
	{
		ExampleAsync(0.1f, Complete);
	}

	FStateGraphRef StateGraph;
};

using FExampleModulePtr = TSharedPtr<FExampleModule, ESPMode::ThreadSafe>;


// FExampleDerived - Derived class that updates the function nodes FExampleBase setup.
class FExampleDerived : public FExampleBase
{
public:

	FExampleDerived(const FStateGraphRef& InStateGraph) :
		FExampleBase(InStateGraph)
	{}

	virtual void SetupStateGraph() override
	{
		FExampleBase::SetupStateGraph();

		// Insert NodeF between NodeC and NodeD
		StateGraph->CreateNode("Derived.NodeF", this, &FExampleDerived::NodeF)
			->Dependencies.Add("Base.NodeC");

		FStateGraphNodeRef* NodeDRef = StateGraph->GetNodeRef("Base.NodeD");
		check(NodeDRef);
		(*NodeDRef)->Dependencies.Add("Derived.NodeF");

		// Add some additional nodes.
		StateGraph->CreateNode("Derived.NodeG", this, [](FStateGraph& InStateGraph, FStateGraphNodeFunctionComplete Complete) { Complete(); })
			->Next<FExampleClassNode>("Derived.NodeH");
	}

protected:

	void NodeF(FStateGraph& InStateGraph, FStateGraphNodeFunctionComplete Complete)
	{
		FExampleModulePtr ModulePtr = MakeShared<FExampleModule>(StateGraph);
		ModulePtr->Run(FExampleModuleComplete::CreateSPLambda(this, [ModulePtr, Complete](int32 Output)
		{
			// Use output...
			Complete();
		}));
	}
};


TEST_CASE("Example demonstrating the features of StateGraph", "[Example]")
{
	// Ignore potenital depercated .ini warnings in this test.
	LogConfig.SetVerbosity(ELogVerbosity::Error);

	// Engine does this, but it's needed here to ensure GConfig is initialized for the test.
	FConfigCacheIni::InitializeConfigSystem();

	FStateGraphRef StateGraph(MakeShared<FStateGraph>("Example"));
	StateGraph->Initialize();

	TSharedRef<FExampleDerived> Example = MakeShared<FExampleDerived>(StateGraph);
	Example->SetupStateGraph();

	StateGraph->Run();

	// Tick the fake async tasks until complete.
	while (StateGraph->GetStatus() == FStateGraph::EStatus::Waiting)
	{
		FTSTicker::GetCoreTicker().Tick(0.1f);
	}

	CHECK(StateGraph->GetStatus() == FStateGraph::EStatus::Completed);
}

} // UE::StateGraphTests
