// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"
#include "StateGraph.h"
#include "TestHarness.h"

namespace UE::StateGraphTests::Unit
{
	
class StateGraphDisableWarningsLog
{
public:
	StateGraphDisableWarningsLog()
		: OldVerbosity(LogStateGraph.GetVerbosity())
	{
		LogStateGraph.SetVerbosity(ELogVerbosity::Error);
	}

	~StateGraphDisableWarningsLog()
	{
		if (OldVerbosity != LogStateGraph.GetVerbosity())
		{
			LogStateGraph.SetVerbosity(OldVerbosity);
		}
	}

	ELogVerbosity::Type OldVerbosity;
};

TEST_CASE("FStateGraph Basic Tests", "[FStateGraph]")
{
	FStateGraphRef StateGraph(MakeShared<FStateGraph>("Test"));
	CHECK(StateGraph->GetName() == "Test");
	CHECK(StateGraph->GetStatus() == FStateGraph::EStatus::NotStarted);
	CHECK(FString(StateGraph->GetStatusName()) == FString(TEXT("NotStarted")));
	StateGraph->Run();
	CHECK(StateGraph->GetStatus() == FStateGraph::EStatus::Completed);
}

class FTestNode : public FStateGraphNode
{
public:
	FTestNode(FName InName) :
		FStateGraphNode(InName)
	{}

	FTestNode(FName InName, const TSet<FName>& InDependencies) :
		FStateGraphNode(InName)
	{
		Dependencies.Append(InDependencies);
	}

	virtual void Start() override {}
	virtual void Removed() override { bWasRemovedCalled = true; }
	virtual void TimedOut() override { bWasTimedOutCalled = true; }
	virtual void UpdateConfig() override
	{
		FStateGraphNode::UpdateConfig();
		GConfig->GetInt(*GetConfigSectionName(), TEXT("CustomConfig"), CustomConfig, GEngineIni);
	}

	bool bWasRemovedCalled = false;
	bool bWasTimedOutCalled = false;
	int32 CustomConfig = 1;
};

using FTestNodeRef = TSharedRef<FTestNode, ESPMode::ThreadSafe>;
using FTestNodePtr = TSharedPtr<FTestNode, ESPMode::ThreadSafe>;
using FTestNodeWeakPtr = TWeakPtr<FTestNode, ESPMode::ThreadSafe>;

TEST_CASE("FStateGraphNode Basic Tests", "[FStateGraphNode]")
{
	FStateGraphRef StateGraph(MakeShared<FStateGraph>("Test"));
	FStateGraphNodeRef TestNodeA = StateGraph->CreateNode<FTestNode>("TestNodeA");
	CHECK(TestNodeA->GetName() == "TestNodeA");
	CHECK(TestNodeA->GetStatus() == FStateGraphNode::EStatus::NotStarted);
	CHECK(FString(TestNodeA->GetStatusName()) == FString(TEXT("NotStarted")));
	CHECK(&TestNodeA.Get() == &StateGraph->GetNodeRef("TestNodeA")->Get());
	CHECK(&TestNodeA.Get() == StateGraph->GetNode<FTestNode>("TestNodeA").Get());

	StateGraph->Run();
	CHECK(StateGraph->GetStatus() == FStateGraph::EStatus::Waiting);
	CHECK(TestNodeA->GetStatus() == FStateGraphNode::EStatus::Started);

	TestNodeA->Complete();
	CHECK(StateGraph->GetStatus() == FStateGraph::EStatus::Completed);
	CHECK(TestNodeA->GetStatus() == FStateGraphNode::EStatus::Completed);
}

TEST_CASE("FStateGraphNodeFunction Basic Tests", "[FStateGraphNodeFunction]")
{
	FStateGraphRef StateGraph(MakeShared<FStateGraph>("Test"));
	FStateGraphNodeFunctionComplete Complete;
	FStateGraphNodeFunctionRef TestNodeA = StateGraph->CreateNode("TestNodeA", [&Complete](FStateGraph& StateGraph, FStateGraphNodeFunctionComplete InComplete) { Complete = InComplete; });
	CHECK(TestNodeA->GetStatus() == FStateGraphNode::EStatus::NotStarted);
	CHECK(FString(TestNodeA->GetStatusName()) == FString(TEXT("NotStarted")));

	StateGraph->Run();
	CHECK(StateGraph->GetStatus() == FStateGraph::EStatus::Waiting);
	CHECK(TestNodeA->GetStatus() == FStateGraphNode::EStatus::Started);

	Complete();
	CHECK(StateGraph->GetStatus() == FStateGraph::EStatus::Completed);
	CHECK(TestNodeA->GetStatus() == FStateGraphNode::EStatus::Completed);
}

TEST_CASE("FStateGraph ContextName", "[FStateGraphContextName]")
{
	FStateGraphRef StateGraph(MakeShared<FStateGraph>("Test", TEXT("TestContextName")));
	FStateGraphNodeFunctionRef TestNodeA = StateGraph->CreateNode("TestNodeA", [](FStateGraph& StateGraph, FStateGraphNodeFunctionComplete Complete) { Complete(); });
	StateGraph->Run();
	CHECK(StateGraph->GetContextName() == TEXT("TestContextName"));
	CHECK(StateGraph->GetLogName() == TEXT("Test(TestContextName)"));
	CHECK(TestNodeA->GetLogName() == TEXT("Test(TestContextName).TestNodeA"));
}

class FClassNode : public FStateGraphNode
{
public:
	FClassNode(FName InName) : FStateGraphNode(InName) {}
	virtual void Start() { Complete(); }
};

void StaticNode(FStateGraph& StateGraph, FStateGraphNodeFunctionComplete Complete) { Complete(); }

class FRawClass
{
public:
	void RawNode(FStateGraph& StateGraph, FStateGraphNodeFunctionComplete Complete) { Complete(); }
};

class FSPClass : public TSharedFromThis<FSPClass>
{
public:
	void SPNode(FStateGraph& StateGraph, FStateGraphNodeFunctionComplete Complete) { Complete(); }
};

TEST_CASE("CreateNode for each supported type", "[CreateNodeFunctions]")
{
	FStateGraphRef StateGraph(MakeShared<FStateGraph>("Test"));
	FRawClass RawClass;
	TSharedRef<FSPClass> SPClass(MakeShared<FSPClass>());

	StateGraph->CreateNode<FClassNode>("TestClass")
		->Next("TestLambda", [](FStateGraph& StateGraph, FStateGraphNodeFunctionComplete Complete) { Complete(); })
		->Next("TestStatic", &StaticNode)
		->Next("TestRaw", &RawClass, &FRawClass::RawNode)
		->Next("TestSP", &SPClass.Get(), &FSPClass::SPNode)
		->Next("TestSPLambda", &SPClass.Get(), [](FStateGraph& StateGraph, FStateGraphNodeFunctionComplete Complete) { Complete(); });

	StateGraph->Run();
	CHECK(StateGraph->GetStatus() == FStateGraph::EStatus::Completed);
}

TEST_CASE("Node dependencies", "[NodeDependencies]")
{
	StateGraphDisableWarningsLog ScopeDisable;

	FStateGraphRef StateGraph(MakeShared<FStateGraph>("Test"));

	FStateGraphNodeRef TestNodeA = StateGraph->CreateNode<FTestNode>("TestNodeA");
	FStateGraphNodeRef TestNodeB = StateGraph->CreateNode<FTestNode>("TestNodeB", TSet<FName>({"TestNodeA"}));
	FStateGraphNodeRef TestNodeC = StateGraph->CreateNode<FTestNode>("TestNodeC", TSet<FName>({"TestNodeB", "TestNodeD"}));

	StateGraph->Run();
	CHECK(StateGraph->GetStatus() == FStateGraph::EStatus::Waiting);
	CHECK(TestNodeA->GetStatus() == FStateGraphNode::EStatus::Started);
	CHECK(TestNodeB->GetStatus() == FStateGraphNode::EStatus::Blocked);
	CHECK(TestNodeC->GetStatus() == FStateGraphNode::EStatus::Blocked);

	TestNodeA->Complete();
	CHECK(StateGraph->GetStatus() == FStateGraph::EStatus::Waiting);
	CHECK(TestNodeA->GetStatus() == FStateGraphNode::EStatus::Completed);
	CHECK(TestNodeB->GetStatus() == FStateGraphNode::EStatus::Started);
	CHECK(TestNodeC->GetStatus() == FStateGraphNode::EStatus::Blocked);

	TestNodeB->Complete();
	CHECK(StateGraph->GetStatus() == FStateGraph::EStatus::Blocked);
	CHECK(TestNodeB->GetStatus() == FStateGraphNode::EStatus::Completed);
	CHECK(TestNodeC->GetStatus() == FStateGraphNode::EStatus::Blocked);

	FStateGraphNodeRef TestNodeD = StateGraph->CreateNode<FTestNode>("TestNodeD");
	StateGraph->Run();
	CHECK(StateGraph->GetStatus() == FStateGraph::EStatus::Waiting);
	CHECK(TestNodeC->GetStatus() == FStateGraphNode::EStatus::Blocked);
	CHECK(TestNodeD->GetStatus() == FStateGraphNode::EStatus::Started);

	TestNodeD->Complete();
	CHECK(StateGraph->GetStatus() == FStateGraph::EStatus::Waiting);
	CHECK(TestNodeC->GetStatus() == FStateGraphNode::EStatus::Started);
	CHECK(TestNodeD->GetStatus() == FStateGraphNode::EStatus::Completed);

	TestNodeC->Complete();
	CHECK(StateGraph->GetStatus() == FStateGraph::EStatus::Completed);
	CHECK(TestNodeC->GetStatus() == FStateGraphNode::EStatus::Completed);
}

TEST_CASE("Blocked graph", "[BlockedGraph]")
{
	StateGraphDisableWarningsLog ScopeDisable;
	FStateGraphRef StateGraph(MakeShared<FStateGraph>("Test"));

	FStateGraphNodeRef TestNodeA = StateGraph->CreateNode<FTestNode>("TestNodeA", TSet<FName>({ "TestNodeB" }));
	FStateGraphNodeRef TestNodeB = StateGraph->CreateNode<FTestNode>("TestNodeB", TSet<FName>({ "TestNodeA" }));

	StateGraph->Run();
	CHECK(StateGraph->GetStatus() == FStateGraph::EStatus::Blocked);
	CHECK(TestNodeA->GetStatus() == FStateGraphNode::EStatus::Blocked);
	CHECK(TestNodeB->GetStatus() == FStateGraphNode::EStatus::Blocked);
}

TEST_CASE("Adding and reusing nodes", "[AddingNodes]")
{
	FStateGraphRef StateGraphA(MakeShared<FStateGraph>("TestB"));
	FStateGraphNodeRef TestNodeA = MakeShared<FTestNode>("TestNodeA");

	CHECK(StateGraphA->AddNode(TestNodeA));
	StateGraphA->Run();
	CHECK(StateGraphA->GetStatus() == FStateGraph::EStatus::Waiting);
	CHECK(TestNodeA->GetStatus() == FStateGraphNode::EStatus::Started);

	StateGraphA->RemoveNode(TestNodeA->GetName());
	StateGraphA->Run();
	CHECK(StateGraphA->GetStatus() == FStateGraph::EStatus::Completed);
	CHECK(TestNodeA->GetStatus() == FStateGraphNode::EStatus::Started);

	FStateGraphRef StateGraphB(MakeShared<FStateGraph>("TestB"));
	CHECK(StateGraphB->AddNode(TestNodeA));
	StateGraphB->Run();
	CHECK(StateGraphB->GetStatus() == FStateGraph::EStatus::Waiting);
	CHECK(TestNodeA->GetStatus() == FStateGraphNode::EStatus::Started);

	TestNodeA->Complete();
	CHECK(StateGraphB->GetStatus() == FStateGraph::EStatus::Completed);
	CHECK(TestNodeA->GetStatus() == FStateGraphNode::EStatus::Completed);

	{
		StateGraphDisableWarningsLog ScopeDisable;

		StateGraphA->Reset();
		CHECK(StateGraphA->GetStatus() == FStateGraph::EStatus::NotStarted);
		CHECK(!StateGraphA->AddNode(TestNodeA));
	}

	StateGraphB->RemoveNode(TestNodeA->GetName());
	CHECK(StateGraphA->AddNode(TestNodeA));
	StateGraphA->Run();
	CHECK(StateGraphA->GetStatus() == FStateGraph::EStatus::Completed);
	CHECK(TestNodeA->GetStatus() == FStateGraphNode::EStatus::Completed);

	{
		StateGraphDisableWarningsLog ScopeDisable;
		FStateGraphNodeRef TestNodeA2 = MakeShared<FTestNode>("TestNodeA");
		CHECK(!StateGraphA->AddNode(TestNodeA2));
	}
}

TEST_CASE("Removing nodes", "[RemoveNodes]")
{
	FStateGraphRef StateGraph(MakeShared<FStateGraph>("Test"));

	FTestNodePtr TestNodeA = StateGraph->CreateNode<FTestNode>("TestNodeA");
	FTestNodeWeakPtr WeakTestNodeA = TestNodeA;
	StateGraph->RemoveNode("TestNodeA");
	CHECK(TestNodeA->GetStatus() == FStateGraphNode::EStatus::NotStarted);
	CHECK(TestNodeA->bWasRemovedCalled);
	TestNodeA.Reset();
	CHECK(!WeakTestNodeA.IsValid());

	WeakTestNodeA = StateGraph->CreateNode<FTestNode>("TestNodeA");
	StateGraph->RemoveNode("TestNodeA");
	CHECK(!WeakTestNodeA.IsValid());

	WeakTestNodeA = StateGraph->CreateNode<FTestNode>("TestNodeA");
	FStateGraphNodePtr TestNodeB = StateGraph->CreateNode("TestNodeB", [](FStateGraph& StateGraph, FStateGraphNodeFunctionComplete Complete)
	{
		StateGraph.RemoveNode("TestNodeA");
		StateGraph.RemoveNode("TestNodeC");
	});
	FStateGraphNodeWeakPtr WeakTestNodeC = StateGraph->CreateNode<FTestNode>("TestNodeC");
	StateGraph->Run();
	CHECK(!WeakTestNodeA.IsValid());
	CHECK(!WeakTestNodeC.IsValid());
	StateGraph->RemoveNode("TestNodeB");

	StateGraph->CreateNode<FTestNode>("TestNodeA")
		->Next<FTestNode>("TestNodeB")
		->Next<FTestNode>("TestNodeC");
	StateGraph->RemoveAllNodes();
	CHECK(StateGraph->GetNodeRef("TestNodeA") == nullptr);
	CHECK(StateGraph->GetNodeRef("TestNodeB") == nullptr);
	CHECK(StateGraph->GetNodeRef("TestNodeC") == nullptr);
}

TEST_CASE("Resetting nodes", "[ResetNodes]")
{
	FStateGraphRef StateGraph(MakeShared<FStateGraph>("Test"));

	int32 StartCount = 0;
	FStateGraphNodeRef TestNodeA = StateGraph->CreateNode("TestNodeA", [&StartCount, &TestNodeA](FStateGraph& StateGraph, FStateGraphNodeFunctionComplete Complete)
	{
		CHECK(TestNodeA->GetStatus() == FStateGraphNode::EStatus::Started);

		if (StartCount == 0)
		{
			TestNodeA->Reset();
		}
		else
		{
			Complete();
		}

		++StartCount;
	});

	StateGraph->Run();
	CHECK(StartCount == 1);
	CHECK(TestNodeA->GetStatus() == FStateGraphNode::EStatus::NotStarted);

	StateGraph->Run();
	CHECK(StartCount == 2);
	CHECK(TestNodeA->GetStatus() == FStateGraphNode::EStatus::Completed);

	TestNodeA->Reset();
	StateGraph->Run();
	CHECK(StartCount == 3);
}

TEST_CASE("Resetting state graph", "[ResetStateGraph]")
{
	FStateGraphRef StateGraph(MakeShared<FStateGraph>("Test"));

	int32 StartCount = 0;
	FStateGraphNodeRef TestNodeA = StateGraph->CreateNode("TestNodeA", [&StartCount](FStateGraph& StateGraph, FStateGraphNodeFunctionComplete Complete)
	{
		if (StartCount == 0)
		{
			StateGraph.Reset();
		}
		else
		{
			Complete();
		}

		++StartCount;
	});

	StateGraph->Run();
	CHECK(StartCount == 1);
	CHECK(TestNodeA->GetStatus() == FStateGraphNode::EStatus::NotStarted);
	CHECK(StateGraph->GetStatus() == FStateGraph::EStatus::NotStarted);

	StateGraph->Run();
	CHECK(StartCount == 2);
	CHECK(TestNodeA->GetStatus() == FStateGraphNode::EStatus::Completed);
	CHECK(StateGraph->GetStatus() == FStateGraph::EStatus::Completed);
}

TEST_CASE("Deleting state graph", "[DeleteStateGraph]")
{
	FStateGraphPtr StateGraph(MakeShared<FStateGraph>("Test"));
	FStateGraphWeakPtr WeakStateGraph(StateGraph);
	FStateGraphNodeRef TestNodeA = StateGraph->CreateNode("TestNodeA", [&StateGraph, &WeakStateGraph](FStateGraph& InStateGraph, FStateGraphNodeFunctionComplete Complete)
	{
		StateGraph.Reset();
		CHECK(!WeakStateGraph.IsValid());
	});

	StateGraph->Run();
	CHECK(!WeakStateGraph.IsValid());
}

TEST_CASE("Pausing state graph", "[PauseStateGraph]")
{
	FStateGraphRef StateGraph(MakeShared<FStateGraph>("Test"));
	FStateGraphNodeRef TestNodeA = StateGraph->CreateNode("TestNodeA", [](FStateGraph& StateGraph, FStateGraphNodeFunctionComplete Complete) { StateGraph.Pause(); });
	FStateGraphNodeRef TestNodeB = StateGraph->CreateNode<FTestNode>("TestNodeB", TSet<FName>({"TestNodeA"}));
	FStateGraphNodeRef TestNodeC = StateGraph->CreateNode("TestNodeC", [](FStateGraph& StateGraph, FStateGraphNodeFunctionComplete Complete) { Complete(); });
	TestNodeC->Dependencies.Add("TestNodeB");

	StateGraph->Run();
	CHECK(StateGraph->GetStatus() == FStateGraph::EStatus::Paused);
	CHECK(TestNodeA->GetStatus() == FStateGraphNode::EStatus::Started);

	TestNodeA->Complete();
	CHECK(StateGraph->GetStatus() == FStateGraph::EStatus::Paused);
	CHECK(TestNodeA->GetStatus() == FStateGraphNode::EStatus::Completed);
	CHECK(TestNodeB->GetStatus() != FStateGraphNode::EStatus::Started);

	StateGraph->Run();
	CHECK(StateGraph->GetStatus() == FStateGraph::EStatus::Waiting);
	CHECK(TestNodeA->GetStatus() == FStateGraphNode::EStatus::Completed);
	CHECK(TestNodeB->GetStatus() == FStateGraphNode::EStatus::Started);
	CHECK(TestNodeC->GetStatus() == FStateGraphNode::EStatus::Blocked);

	StateGraph->Pause();
	TestNodeB->Complete();
	CHECK(StateGraph->GetStatus() == FStateGraph::EStatus::Paused);
	CHECK(TestNodeB->GetStatus() == FStateGraphNode::EStatus::Completed);
	CHECK(TestNodeC->GetStatus() == FStateGraphNode::EStatus::Blocked);

	StateGraph->Run();
	CHECK(StateGraph->GetStatus() == FStateGraph::EStatus::Completed);
	CHECK(TestNodeC->GetStatus() == FStateGraphNode::EStatus::Completed);
}

TEST_CASE("State graph timeout", "[StateGraphTimeout]")
{
	FStateGraphRef StateGraph(MakeShared<FStateGraph>("Test"));
	StateGraph->SetTimeout(0.01);
	bool bOnTimedOutCalled = false;
	StateGraph->OnStatusChanged.AddLambda([&bOnTimedOutCalled](FStateGraph&, FStateGraph::EStatus OldStatus, FStateGraph::EStatus NewStatus)
	{
		if (NewStatus == FStateGraph::EStatus::TimedOut)
		{
			bOnTimedOutCalled = true;
		}
	});

	FStateGraphNodeRef TestNodeA = StateGraph->CreateNode("TestNodeA", [](FStateGraph& StateGraph, FStateGraphNodeFunctionComplete Complete) {});
	StateGraph->Run();

	while (StateGraph->GetStatus() == FStateGraph::EStatus::Waiting)
	{
		FTSTicker::GetCoreTicker().Tick(0.1f);
	}

	CHECK(StateGraph->GetStatus() == FStateGraph::EStatus::TimedOut);
	CHECK(bOnTimedOutCalled);
}

TEST_CASE("Node timeout", "[NodeTimeout]")
{
	FStateGraphRef StateGraph(MakeShared<FStateGraph>("Test"));
	bool bOnNodeTimedOutCalled = false;
	StateGraph->OnNodeStatusChanged.AddLambda([&bOnNodeTimedOutCalled](FStateGraphNode&, FStateGraphNode::EStatus OldStatus, FStateGraphNode::EStatus NewStatus)
	{
		if (NewStatus == FStateGraphNode::EStatus::TimedOut)
		{
			bOnNodeTimedOutCalled = true;
		}
	});

	FTestNodeRef TestNodeA = StateGraph->CreateNode<FTestNode>("TestNodeA");
	TestNodeA->SetTimeout(0.01);
	StateGraph->Run();

	while (StateGraph->GetStatus() == FStateGraph::EStatus::Waiting)
	{
		StateGraphDisableWarningsLog ScopeDisable;
		FTSTicker::GetCoreTicker().Tick(0.1f);
	}

	CHECK(TestNodeA->GetStatus() == FStateGraphNode::EStatus::TimedOut);
	CHECK(StateGraph->GetStatus() == FStateGraph::EStatus::Blocked);
	CHECK(TestNodeA->bWasTimedOutCalled);
	CHECK(bOnNodeTimedOutCalled);

	TestNodeA->Complete();
	CHECK(StateGraph->GetStatus() == FStateGraph::EStatus::Completed);
}

TEST_CASE("State graph config", "[StateGraphConfig]")
{
	StateGraphDisableWarningsLog ScopeDisable;
	LogConfig.SetVerbosity(ELogVerbosity::Error);
	FConfigCacheIni::InitializeConfigSystem();
	FConfigFile* EngineIni = GConfig->FindConfigFile(GEngineIni);
	check(EngineIni);
	EngineIni->CombineFromBuffer(TEXT("[StateGraph.Test]\nTimeout=0.01\n"), GEngineIni);

	FStateGraphRef StateGraph(MakeShared<FStateGraph>("Test"));
	StateGraph->Initialize();
	FStateGraphNodeRef TestNodeA = StateGraph->CreateNode("TestNodeA", [](FStateGraph& StateGraph, FStateGraphNodeFunctionComplete Complete) {});
	StateGraph->Run();

	while (StateGraph->GetStatus() == FStateGraph::EStatus::Waiting)
	{
		FTSTicker::GetCoreTicker().Tick(0.1f);
	}

	CHECK(StateGraph->GetStatus() == FStateGraph::EStatus::TimedOut);
	TestNodeA->Complete();
	CHECK(StateGraph->GetStatus() == FStateGraph::EStatus::TimedOut);

	EngineIni->CombineFromBuffer(TEXT("[StateGraph.Test]\nTimeout=0\n"), GEngineIni);
	FCoreDelegates::TSOnConfigSectionsChanged().Broadcast(GEngineIni, TSet<FString>({ TEXT("StateGraph.Test") }));

	StateGraph->Run();
	CHECK(StateGraph->GetStatus() == FStateGraph::EStatus::Completed);
}

TEST_CASE("Node config", "[NodeConfig]")
{
	StateGraphDisableWarningsLog ScopeDisable;
	LogConfig.SetVerbosity(ELogVerbosity::Error);
	FConfigCacheIni::InitializeConfigSystem();
	FConfigFile* EngineIni = GConfig->FindConfigFile(GEngineIni);
	check(EngineIni);
	EngineIni->CombineFromBuffer(TEXT("[StateGraph.Test.TestNodeA]\nTimeout=0.01\nCustomConfig=2\n"), GEngineIni);

	FStateGraphRef StateGraph(MakeShared<FStateGraph>("Test"));
	StateGraph->Initialize();
	FTestNodeRef TestNodeA = StateGraph->CreateNode<FTestNode>("TestNodeA");
	StateGraph->Run();

	while (StateGraph->GetStatus() == FStateGraph::EStatus::Waiting)
	{
		FTSTicker::GetCoreTicker().Tick(0.1f);
	}

	CHECK(TestNodeA->GetStatus() == FStateGraphNode::EStatus::TimedOut);
	CHECK(StateGraph->GetStatus() == FStateGraph::EStatus::Blocked);
	CHECK(TestNodeA->bWasTimedOutCalled);
	CHECK(TestNodeA->CustomConfig == 2);

	EngineIni->CombineFromBuffer(TEXT("[StateGraph.Test.TestNodeA]\nTimeout=0.01\nCustomConfig=3\n"), GEngineIni);
	FCoreDelegates::TSOnConfigSectionsChanged().Broadcast(GEngineIni, TSet<FString>({ TEXT("StateGraph.Test.TestNodeA") }));

	CHECK(TestNodeA->CustomConfig == 3);
}

} // UE::StateGraphTests
