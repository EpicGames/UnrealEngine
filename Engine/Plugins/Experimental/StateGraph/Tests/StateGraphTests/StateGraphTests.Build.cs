// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class StateGraphTests : TestModuleRules
{
	static StateGraphTests()
	{
		TestMetadata = new Metadata();
		TestMetadata.TestName = "StateGraph";
		TestMetadata.TestShortName = "State Graph";
	}

	public StateGraphTests(ReadOnlyTargetRules Target) : base(Target)
	{
		OptimizeCode = CodeOptimization.Never;

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"StateGraph"
			});
	}
}
