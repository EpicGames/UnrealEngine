// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "TestHarness.h"

// Other includes must be placed after CoreMinimal.h and TestHarness.h, grouped by scope (std libraries, UE modules, third party etc)
#include "AnimNextConfig.h"

#include <catch2/catch_test_macros.hpp>
#include <catch2/generators/catch_generators.hpp>

/* Classic TDD-style test */

TEST_CASE("AnimNextConfig Test Case", "[UAF][unit][MustPass][LiveTest]")
{
	// Setup code for this test case

	// Test can be divided into sections
	SECTION("GetAllowedAssetClasses Test")
	{
		TArray<UClass*> AllowedAssetClasses = UAnimNextConfig::GetAllowedAssetClasses();
		REQUIRE(AllowedAssetClasses.Num() > 0);
	}

	// Teardown code for this test case
}
