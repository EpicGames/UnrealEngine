// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "TestHarness.h"
#include "BindableValue/UAFBindableTypes.h"
#include "BindableValue/UAFPropertyBinding.h"
#include "UAFAssetInstance.h"
#include "Variables/AnimNextVariableReference.h"
#include "Variables/UAFInstanceVariableData.h"
#include "UAFTestVars.h"

#include <catch2/catch_test_macros.hpp>
#include <catch2/benchmark/catch_benchmark.hpp>

// ---------------------------------------------------------------------------
// Helpers
// ---------------------------------------------------------------------------

namespace UE::UAF::Tests
{
	// Forward-declared friend in FUAFAssetInstance — reuse for perf tests.
	struct FUAFBindingTestFixture
	{
		static void InitFromStruct(FUAFAssetInstance& InInstance, const UScriptStruct* InStruct)
		{
			InInstance.Variables.AddVariablesContainerForStruct(InStruct, InInstance, nullptr);
			InInstance.Variables.RebuildNameMaps();
		}
	};
} // namespace UE::UAF::Tests

namespace
{
	void InitPerfInstance(FUAFAssetInstance& Instance)
	{
		UE::UAF::Tests::FUAFBindingTestFixture::InitFromStruct(Instance, FUAFPerfVars10::StaticStruct());
	}

	FAnimNextVariableReference MakePerfRef(FName PropName)
	{
		const FProperty* Prop = FUAFPerfVars10::StaticStruct()->FindPropertyByName(PropName);
		check(Prop);
		return FAnimNextVariableReference::FromProperty(Prop, FUAFPerfVars10::StaticStruct());
	}

	FUAFPropertyBinding MakeVarBinding(const FAnimNextVariableReference& SourceVariable)
	{
		FUAFPropertyBinding Binding;
		Binding.SourceType     = EUAFBindingSourceType::Variable;
		Binding.SourceVariable = SourceVariable;
		return Binding;
	}

	FUAFPropertyBinding MakeSubPropBinding(const FAnimNextVariableReference& SourceVariable, FStringView SubPath)
	{
		FPropertyBindingPath Path;
		const bool bParsed = Path.FromString(SubPath);
		REQUIRE(bParsed);

		FUAFPropertyBinding Binding;
		Binding.SourceType      = EUAFBindingSourceType::SubProperty;
		Binding.SourceVariable  = SourceVariable;
		Binding.SubPropertyPath = MoveTemp(Path);
		return Binding;
	}

} // anonymous namespace

// ---------------------------------------------------------------------------
// Direct (variable binding) benchmarks
// ---------------------------------------------------------------------------

TEST_CASE("FBindableBool.BulkVariableBinding", "[UAF][perf]")
{
	FUAFAssetInstance Owner;
	InitPerfInstance(Owner);

	FAnimNextVariableReference Refs[10];
	for (int i = 0; i < 10; ++i)
	{
		Refs[i] = MakePerfRef(*FString::Printf(TEXT("bBool%d"), i));
	}

	FBindableBool Fields[10];
	for (int i = 0; i < 10; ++i)
	{
		Fields[i].SetBinding(MakeVarBinding(Refs[i]));
	}

	BENCHMARK("10x FBindableBool variable read")
	{
		int32 Sum = 0;
		for (int i = 0; i < 10; ++i)
		{
			Sum += Fields[i].GetValue(&Owner) ? 1 : 0;
		}
		return Sum;
	};
}

TEST_CASE("FBindableFloat.BulkVariableBinding", "[UAF][perf]")
{
	FUAFAssetInstance Owner;
	InitPerfInstance(Owner);

	FAnimNextVariableReference Refs[10];
	for (int i = 0; i < 10; ++i)
	{
		Refs[i] = MakePerfRef(*FString::Printf(TEXT("f%d"), i));
	}

	FBindableFloat Fields[10];
	for (int i = 0; i < 10; ++i)
	{
		Fields[i].SetBinding(MakeVarBinding(Refs[i]));
	}

	BENCHMARK("10x FBindableFloat variable read")
	{
		float Sum = 0.f;
		for (int i = 0; i < 10; ++i)
		{
			Sum += Fields[i].GetValue(&Owner);
		}
		return Sum;
	};
}

TEST_CASE("FBindableDouble.BulkVariableBinding", "[UAF][perf]")
{
	FUAFAssetInstance Owner;
	InitPerfInstance(Owner);

	FAnimNextVariableReference Refs[10];
	for (int i = 0; i < 10; ++i)
	{
		Refs[i] = MakePerfRef(*FString::Printf(TEXT("d%d"), i));
	}

	FBindableDouble Fields[10];
	for (int i = 0; i < 10; ++i)
	{
		Fields[i].SetBinding(MakeVarBinding(Refs[i]));
	}

	BENCHMARK("10x FBindableDouble variable read")
	{
		double Sum = 0.0;
		for (int i = 0; i < 10; ++i)
		{
			Sum += Fields[i].GetValue(&Owner);
		}
		return Sum;
	};
}

TEST_CASE("FBindableInt32.BulkVariableBinding", "[UAF][perf]")
{
	FUAFAssetInstance Owner;
	InitPerfInstance(Owner);

	FAnimNextVariableReference Refs[10];
	for (int i = 0; i < 10; ++i)
	{
		Refs[i] = MakePerfRef(*FString::Printf(TEXT("i%d"), i));
	}

	FBindableInt32 Fields[10];
	for (int i = 0; i < 10; ++i)
	{
		Fields[i].SetBinding(MakeVarBinding(Refs[i]));
	}

	BENCHMARK("10x FBindableInt32 variable read")
	{
		int32 Sum = 0;
		for (int i = 0; i < 10; ++i)
		{
			Sum += Fields[i].GetValue(&Owner);
		}
		return Sum;
	};
}

TEST_CASE("FBindableInt64.BulkVariableBinding", "[UAF][perf]")
{
	FUAFAssetInstance Owner;
	InitPerfInstance(Owner);

	FAnimNextVariableReference Refs[10];
	for (int i = 0; i < 10; ++i)
	{
		Refs[i] = MakePerfRef(*FString::Printf(TEXT("l%d"), i));
	}

	FBindableInt64 Fields[10];
	for (int i = 0; i < 10; ++i)
	{
		Fields[i].SetBinding(MakeVarBinding(Refs[i]));
	}

	BENCHMARK("10x FBindableInt64 variable read")
	{
		int64 Sum = 0;
		for (int i = 0; i < 10; ++i)
		{
			Sum += Fields[i].GetValue(&Owner);
		}
		return Sum;
	};
}

TEST_CASE("FBindableByte.BulkVariableBinding", "[UAF][perf]")
{
	FUAFAssetInstance Owner;
	InitPerfInstance(Owner);

	FAnimNextVariableReference Refs[10];
	for (int i = 0; i < 10; ++i)
	{
		Refs[i] = MakePerfRef(*FString::Printf(TEXT("u%d"), i));
	}

	FBindableByte Fields[10];
	for (int i = 0; i < 10; ++i)
	{
		Fields[i].SetBinding(MakeVarBinding(Refs[i]));
	}

	BENCHMARK("10x FBindableByte variable read")
	{
		uint32 Sum = 0;
		for (int i = 0; i < 10; ++i)
		{
			Sum += Fields[i].GetValue(&Owner);
		}
		return Sum;
	};
}

TEST_CASE("FBindableName.BulkVariableBinding", "[UAF][perf]")
{
	FUAFAssetInstance Owner;
	InitPerfInstance(Owner);

	FAnimNextVariableReference Refs[10];
	for (int i = 0; i < 10; ++i)
	{
		Refs[i] = MakePerfRef(*FString::Printf(TEXT("nm%d"), i));
	}

	FBindableName Fields[10];
	for (int i = 0; i < 10; ++i)
	{
		Fields[i].SetBinding(MakeVarBinding(Refs[i]));
	}

	BENCHMARK("10x FBindableName variable read")
	{
		FName Last;
		for (int i = 0; i < 10; ++i)
		{
			Last = Fields[i].GetValue(&Owner);
		}
		return Last;
	};
}

TEST_CASE("FBindableEnum.BulkVariableBinding", "[UAF][perf]")
{
	FUAFAssetInstance Owner;
	InitPerfInstance(Owner);

	FAnimNextVariableReference Refs[10];
	for (int i = 0; i < 10; ++i)
	{
		Refs[i] = MakePerfRef(*FString::Printf(TEXT("en%d"), i));
	}

	FBindableEnum Fields[10];
	for (int i = 0; i < 10; ++i)
	{
		Fields[i].SetConstantValue((int32)EUAFTestEnum::ValueA);
		Fields[i].SetEnumClass(StaticEnum<EUAFTestEnum>());
		Fields[i].SetBinding(MakeVarBinding(Refs[i]));
	}

	BENCHMARK("10x FBindableEnum variable read")
	{
		int32 Sum = 0;
		for (int i = 0; i < 10; ++i)
		{
			Sum += Fields[i].GetValue(&Owner);
		}
		return Sum;
	};
}

TEST_CASE("FBindableVector.BulkVariableBinding", "[UAF][perf]")
{
	FUAFAssetInstance Owner;
	InitPerfInstance(Owner);

	FAnimNextVariableReference Refs[10];
	for (int i = 0; i < 10; ++i)
	{
		Refs[i] = MakePerfRef(*FString::Printf(TEXT("v%d"), i));
	}

	FBindableVector Fields[10];
	for (int i = 0; i < 10; ++i)
	{
		Fields[i].SetBinding(MakeVarBinding(Refs[i]));
	}

	BENCHMARK("10x FBindableVector variable read")
	{
		FVector Last = FVector::ZeroVector;
		for (int i = 0; i < 10; ++i)
		{
			Last = Fields[i].GetValue(&Owner);
		}
		return Last;
	};
}

TEST_CASE("FBindableQuat.BulkVariableBinding", "[UAF][perf]")
{
	FUAFAssetInstance Owner;
	InitPerfInstance(Owner);

	FAnimNextVariableReference Refs[10];
	for (int i = 0; i < 10; ++i)
	{
		Refs[i] = MakePerfRef(*FString::Printf(TEXT("q%d"), i));
	}

	FBindableQuat Fields[10];
	for (int i = 0; i < 10; ++i)
	{
		Fields[i].SetBinding(MakeVarBinding(Refs[i]));
	}

	BENCHMARK("10x FBindableQuat variable read")
	{
		FQuat Last = FQuat::Identity;
		for (int i = 0; i < 10; ++i)
		{
			Last = Fields[i].GetValue(&Owner);
		}
		return Last;
	};
}

// ---------------------------------------------------------------------------
// Sub-property binding benchmarks
// ---------------------------------------------------------------------------

TEST_CASE("FBindableFloat.BulkSubPropertyBinding", "[UAF][perf]")
{
	FUAFAssetInstance Owner;
	InitPerfInstance(Owner);

	FAnimNextVariableReference Refs[10];
	for (int i = 0; i < 10; ++i)
	{
		Refs[i] = MakePerfRef(*FString::Printf(TEXT("v%d"), i));
	}

	FBindableFloat Fields[10];
	for (int i = 0; i < 10; ++i)
	{
		Fields[i].SetBinding(MakeSubPropBinding(Refs[i], TEXT("X")));
	}

	BENCHMARK("10x FBindableFloat sub-property read")
	{
		float Sum = 0.f;
		for (int i = 0; i < 10; ++i)
		{
			Sum += Fields[i].GetValue(&Owner);
		}
		return Sum;
	};
}

TEST_CASE("FBindableDouble.BulkSubPropertyBinding", "[UAF][perf]")
{
	FUAFAssetInstance Owner;
	InitPerfInstance(Owner);

	FAnimNextVariableReference Refs[10];
	for (int i = 0; i < 10; ++i)
	{
		Refs[i] = MakePerfRef(*FString::Printf(TEXT("v%d"), i));
	}

	FBindableDouble Fields[10];
	for (int i = 0; i < 10; ++i)
	{
		Fields[i].SetBinding(MakeSubPropBinding(Refs[i], TEXT("X")));
	}

	BENCHMARK("10x FBindableDouble sub-property read")
	{
		double Sum = 0.0;
		for (int i = 0; i < 10; ++i)
		{
			Sum += Fields[i].GetValue(&Owner);
		}
		return Sum;
	};
}

TEST_CASE("FBindableVector.BulkSubPropertyBinding", "[UAF][perf]")
{
	FUAFAssetInstance Owner;
	InitPerfInstance(Owner);

	FAnimNextVariableReference Refs[10];
	for (int i = 0; i < 10; ++i)
	{
		Refs[i] = MakePerfRef(*FString::Printf(TEXT("ns%d"), i));
	}

	FBindableVector Fields[10];
	for (int i = 0; i < 10; ++i)
	{
		Fields[i].SetBinding(MakeSubPropBinding(Refs[i], TEXT("Vec")));
	}

	BENCHMARK("10x FBindableVector sub-property read")
	{
		FVector Last = FVector::ZeroVector;
		for (int i = 0; i < 10; ++i)
		{
			Last = Fields[i].GetValue(&Owner);
		}
		return Last;
	};
}
