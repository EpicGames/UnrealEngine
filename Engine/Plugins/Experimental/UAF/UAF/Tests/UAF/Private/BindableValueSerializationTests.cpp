// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "TestHarness.h"

#include "BindableValue/UAFBindableTypes.h"
#include "BindableValue/UAFPropertyBinding.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/ObjectAndNameAsStringProxyArchive.h"
#include "UAFTestVars.h"
#include "Variables/AnimNextVariableReference.h"

#include <catch2/catch_test_macros.hpp>

// ---------------------------------------------------------------------------
// Helpers
// ---------------------------------------------------------------------------

namespace
{

FUAFPropertyBinding MakeTestBinding()
{
	const FProperty* Prop = FUAFTestVars::StaticStruct()->FindPropertyByName(TEXT("FloatVal"));
	check(Prop);

	FUAFPropertyBinding Binding;
	Binding.SourceType     = EUAFBindingSourceType::Variable;
	Binding.SourceVariable = FAnimNextVariableReference::FromProperty(Prop, FUAFTestVars::StaticStruct());
	return Binding;
}

// ---------------------------------------------------------------------------
// Binary serialization helpers
// ---------------------------------------------------------------------------

template<typename BindableType>
BindableType BinaryRoundTrip(const BindableType& Source)
{
	// Serialize — use FObjectAndNameAsStringProxyArchive to handle UObject* references as strings
	TArray<uint8> Buffer;
	FMemoryWriter MemWriter(Buffer);
	FObjectAndNameAsStringProxyArchive Writer(MemWriter, false);
	UScriptStruct* Struct = BindableType::StaticStruct();
	Struct->SerializeItem(Writer, const_cast<BindableType*>(&Source), nullptr);

	// Deserialize
	BindableType Dest;
	FMemoryReader MemReader(Buffer);
	FObjectAndNameAsStringProxyArchive Reader(MemReader, false);
	Struct->SerializeItem(Reader, &Dest, nullptr);
	return Dest;
}

// ---------------------------------------------------------------------------
// Text serialization helpers
// ---------------------------------------------------------------------------

template<typename BindableType>
BindableType TextRoundTrip(const BindableType& Source)
{
	// Export
	FString Text;
	BindableType Default;
	Source.ExportTextItem(Text, Default, nullptr, 0, nullptr);

	// Import
	BindableType Dest;
	const TCHAR* Buffer = *Text;
	const bool bImported = Dest.ImportTextItem(Buffer, 0, nullptr, nullptr);
	REQUIRE(bImported);
	return Dest;
}

// ---------------------------------------------------------------------------
// Editor-style text injection helper (mimics SetBindingViaText)
// ---------------------------------------------------------------------------

/** Mimics the editor flow: set binding on raw data, export, import back. */
template<typename BindableType>
BindableType EditorStyleSetBinding(const BindableType& Source, const FUAFPropertyBinding& InBinding)
{
	// Step 1: Copy source and set binding (like EnumerateRawData + SetBinding)
	BindableType WithBinding = Source;
	WithBinding.SetBinding(InBinding);

	// Step 2: Export to text (like GetValueAsFormattedString → ExportTextItem)
	FString TextValue;
	BindableType Default;
	WithBinding.ExportTextItem(TextValue, Default, nullptr, 0, nullptr);

	// Step 3: Import back (like SetValueFromFormattedString → ImportText_Direct)
	BindableType Dest;
	const TCHAR* Buffer = *TextValue;
	const bool bImported = Dest.ImportTextItem(Buffer, 0, nullptr, nullptr);
	REQUIRE(bImported);
	return Dest;
}

/** Mimics the editor flow: copy struct, clear binding on copy, export, import back.
 *  This matches the real editor path: EnumerateRawData + ClearBinding + NotifyPostChange.
 *  The exported text will contain __Binding=() (the always-emit empty marker), and ImportTextItem
 *  treats that as no-binding. */
template<typename BindableType>
BindableType EditorStyleClearBinding(const BindableType& Source)
{
	// Step 1: Copy source and clear binding (like EnumerateRawData + ClearBinding on a copy)
	BindableType WithCleared = Source;
	WithCleared.ClearBinding();

	// Step 2: Export to text (like GetValueAsFormattedString -> ExportTextItem)
	FString TextValue;
	BindableType Default;
	WithCleared.ExportTextItem(TextValue, Default, nullptr, 0, nullptr);

	// Step 3: Import back (like SetValueFromFormattedString -> ImportText_Direct)
	BindableType Dest;
	const TCHAR* Buffer = *TextValue;
	const bool bImported = Dest.ImportTextItem(Buffer, 0, nullptr, nullptr);
	REQUIRE(bImported);
	return Dest;
}

} // anonymous namespace

// ===========================================================================
// Binary Serialize round-trip tests
// ===========================================================================

TEST_CASE("FBindableBool.BinarySerialize.WithBinding", "[UAF][unit][MustPass]")
{
	FBindableBool Src(true);
	Src.SetBinding(MakeTestBinding());
	FBindableBool Dst = BinaryRoundTrip(Src);
	CHECK(Dst.GetConstantValue() == true);
	CHECK(Dst.HasBinding());
}

TEST_CASE("FBindableBool.BinarySerialize.NoBinding", "[UAF][unit][MustPass]")
{
	FBindableBool Src(true);
	FBindableBool Dst = BinaryRoundTrip(Src);
	CHECK(Dst.GetConstantValue() == true);
	CHECK_FALSE(Dst.HasBinding());
}

TEST_CASE("FBindableFloat.BinarySerialize.WithBinding", "[UAF][unit][MustPass]")
{
	FBindableFloat Src(42.5f);
	Src.SetBinding(MakeTestBinding());
	FBindableFloat Dst = BinaryRoundTrip(Src);
	CHECK(Dst.GetConstantValue() == 42.5f);
	CHECK(Dst.HasBinding());
}

TEST_CASE("FBindableFloat.BinarySerialize.NoBinding", "[UAF][unit][MustPass]")
{
	FBindableFloat Src(42.5f);
	FBindableFloat Dst = BinaryRoundTrip(Src);
	CHECK(Dst.GetConstantValue() == 42.5f);
	CHECK_FALSE(Dst.HasBinding());
}

TEST_CASE("FBindableDouble.BinarySerialize.WithBinding", "[UAF][unit][MustPass]")
{
	FBindableDouble Src(99.9);
	Src.SetBinding(MakeTestBinding());
	FBindableDouble Dst = BinaryRoundTrip(Src);
	CHECK(Dst.GetConstantValue() == 99.9);
	CHECK(Dst.HasBinding());
}

TEST_CASE("FBindableInt32.BinarySerialize.WithBinding", "[UAF][unit][MustPass]")
{
	FBindableInt32 Src(123);
	Src.SetBinding(MakeTestBinding());
	FBindableInt32 Dst = BinaryRoundTrip(Src);
	CHECK(Dst.GetConstantValue() == 123);
	CHECK(Dst.HasBinding());
}

TEST_CASE("FBindableInt64.BinarySerialize.WithBinding", "[UAF][unit][MustPass]")
{
	FBindableInt64 Src(9876543210LL);
	Src.SetBinding(MakeTestBinding());
	FBindableInt64 Dst = BinaryRoundTrip(Src);
	CHECK(Dst.GetConstantValue() == 9876543210LL);
	CHECK(Dst.HasBinding());
}

TEST_CASE("FBindableByte.BinarySerialize.WithBinding", "[UAF][unit][MustPass]")
{
	FBindableByte Src(42);
	Src.SetBinding(MakeTestBinding());
	FBindableByte Dst = BinaryRoundTrip(Src);
	CHECK(Dst.GetConstantValue() == 42);
	CHECK(Dst.HasBinding());
}

TEST_CASE("FBindableName.BinarySerialize.WithBinding", "[UAF][unit][MustPass]")
{
	FBindableName Src(FName(TEXT("TestName")));
	Src.SetBinding(MakeTestBinding());
	FBindableName Dst = BinaryRoundTrip(Src);
	CHECK(Dst.GetConstantValue() == FName(TEXT("TestName")));
	CHECK(Dst.HasBinding());
}

TEST_CASE("FBindableVector.BinarySerialize.WithBinding", "[UAF][unit][MustPass]")
{
	FBindableVector Src(FVector(1.0, 2.0, 3.0));
	Src.SetBinding(MakeTestBinding());
	FBindableVector Dst = BinaryRoundTrip(Src);
	CHECK(Dst.GetConstantValue() == FVector(1.0, 2.0, 3.0));
	CHECK(Dst.HasBinding());
}

TEST_CASE("FBindableQuat.BinarySerialize.WithBinding", "[UAF][unit][MustPass]")
{
	FBindableQuat Src(FQuat(1.0, 2.0, 3.0, 4.0));
	Src.SetBinding(MakeTestBinding());
	FBindableQuat Dst = BinaryRoundTrip(Src);
	CHECK(Dst.GetConstantValue().X == 1.0);
	CHECK(Dst.GetConstantValue().W == 4.0);
	CHECK(Dst.HasBinding());
}

TEST_CASE("FBindableTransform.BinarySerialize.WithBinding", "[UAF][unit][MustPass]")
{
	FBindableTransform Src(FTransform(FQuat(1.0, 2.0, 3.0, 4.0), FVector(5.0, 6.0, 7.0)));
	Src.SetBinding(MakeTestBinding());
	FBindableTransform Dst = BinaryRoundTrip(Src);
	CHECK(Dst.GetConstantValue().GetTranslation().X == 5.0);
	CHECK(Dst.HasBinding());
}

TEST_CASE("FBindableEnum.BinarySerialize.WithBinding", "[UAF][unit][MustPass]")
{
	FBindableEnum Src(EUAFTestEnum::ValueB);
	Src.SetBinding(MakeTestBinding());
	FBindableEnum Dst = BinaryRoundTrip(Src);
	CHECK(Dst.GetConstantValue() == static_cast<int32>(EUAFTestEnum::ValueB));
	CHECK(Dst.GetEnumClass() == StaticEnum<EUAFTestEnum>());
	CHECK(Dst.HasBinding());
}

TEST_CASE("FBindableStruct.BinarySerialize.WithBinding", "[UAF][unit][MustPass]")
{
	FBindableStruct Src(FUAFTestVars::StaticStruct());
	Src.SetBinding(MakeTestBinding());
	FBindableStruct Dst = BinaryRoundTrip(Src);
	CHECK(Dst.GetStructClass() == FUAFTestVars::StaticStruct());
	CHECK(Dst.HasBinding());
}

TEST_CASE("FBindableObject.BinarySerialize.WithBinding", "[UAF][unit][MustPass]")
{
	FBindableObject Src(UObject::StaticClass());
	Src.SetBinding(MakeTestBinding());
	FBindableObject Dst = BinaryRoundTrip(Src);
	CHECK(Dst.GetObjectClass() == UObject::StaticClass());
	CHECK(Dst.HasBinding());
}

// ===========================================================================
// Text ExportTextItem/ImportTextItem round-trip tests
// ===========================================================================

TEST_CASE("FBindableBool.TextSerialize.WithBinding", "[UAF][unit][MustPass]")
{
	FBindableBool Src(true);
	Src.SetBinding(MakeTestBinding());
	FBindableBool Dst = TextRoundTrip(Src);
	CHECK(Dst.GetConstantValue() == true);
	CHECK(Dst.HasBinding());
}

TEST_CASE("FBindableBool.TextSerialize.NoBinding", "[UAF][unit][MustPass]")
{
	FBindableBool Src(true);

	FString Text;
	FBindableBool Default;
	Src.ExportTextItem(Text, Default, nullptr, 0, nullptr);
	// __Binding=() must ALWAYS be emitted (even without a binding) so that the RigVM
	// controller's text comparison detects binding state changes. See ExportBindingText.
	CHECK(Text.Contains(TEXT("__Binding=()")));

	FBindableBool Dst = TextRoundTrip(Src);
	CHECK(Dst.GetConstantValue() == true);
	CHECK_FALSE(Dst.HasBinding());
}

TEST_CASE("FBindableFloat.TextSerialize.WithBinding", "[UAF][unit][MustPass]")
{
	FBindableFloat Src(42.5f);
	Src.SetBinding(MakeTestBinding());
	FBindableFloat Dst = TextRoundTrip(Src);
	CHECK(Dst.GetConstantValue() == 42.5f);
	CHECK(Dst.HasBinding());
}

TEST_CASE("FBindableDouble.TextSerialize.WithBinding", "[UAF][unit][MustPass]")
{
	FBindableDouble Src(99.9);
	Src.SetBinding(MakeTestBinding());
	FBindableDouble Dst = TextRoundTrip(Src);
	CHECK(Dst.GetConstantValue() == 99.9);
	CHECK(Dst.HasBinding());
}

TEST_CASE("FBindableInt32.TextSerialize.WithBinding", "[UAF][unit][MustPass]")
{
	FBindableInt32 Src(123);
	Src.SetBinding(MakeTestBinding());
	FBindableInt32 Dst = TextRoundTrip(Src);
	CHECK(Dst.GetConstantValue() == 123);
	CHECK(Dst.HasBinding());
}

TEST_CASE("FBindableInt64.TextSerialize.WithBinding", "[UAF][unit][MustPass]")
{
	FBindableInt64 Src(9876543210LL);
	Src.SetBinding(MakeTestBinding());
	FBindableInt64 Dst = TextRoundTrip(Src);
	CHECK(Dst.GetConstantValue() == 9876543210LL);
	CHECK(Dst.HasBinding());
}

TEST_CASE("FBindableByte.TextSerialize.WithBinding", "[UAF][unit][MustPass]")
{
	FBindableByte Src(42);
	Src.SetBinding(MakeTestBinding());
	FBindableByte Dst = TextRoundTrip(Src);
	CHECK(Dst.GetConstantValue() == 42);
	CHECK(Dst.HasBinding());
}

TEST_CASE("FBindableName.TextSerialize.WithBinding", "[UAF][unit][MustPass]")
{
	FBindableName Src(FName(TEXT("TestName")));
	Src.SetBinding(MakeTestBinding());
	FBindableName Dst = TextRoundTrip(Src);
	CHECK(Dst.GetConstantValue() == FName(TEXT("TestName")));
	CHECK(Dst.HasBinding());
}

TEST_CASE("FBindableVector.TextSerialize.WithBinding", "[UAF][unit][MustPass]")
{
	FBindableVector Src(FVector(1.0, 2.0, 3.0));
	Src.SetBinding(MakeTestBinding());
	FBindableVector Dst = TextRoundTrip(Src);
	CHECK(Dst.GetConstantValue() == FVector(1.0, 2.0, 3.0));
	CHECK(Dst.HasBinding());
}

TEST_CASE("FBindableQuat.TextSerialize.WithBinding", "[UAF][unit][MustPass]")
{
	FBindableQuat Src(FQuat(1.0, 2.0, 3.0, 4.0));
	Src.SetBinding(MakeTestBinding());
	FBindableQuat Dst = TextRoundTrip(Src);
	CHECK(Dst.GetConstantValue().X == 1.0);
	CHECK(Dst.GetConstantValue().W == 4.0);
	CHECK(Dst.HasBinding());
}

TEST_CASE("FBindableTransform.TextSerialize.WithBinding", "[UAF][unit][MustPass]")
{
	FBindableTransform Src(FTransform(FQuat(1.0, 2.0, 3.0, 4.0), FVector(5.0, 6.0, 7.0)));
	Src.SetBinding(MakeTestBinding());
	FBindableTransform Dst = TextRoundTrip(Src);
	CHECK(Dst.GetConstantValue().GetTranslation().X == 5.0);
	CHECK(Dst.HasBinding());
}

TEST_CASE("FBindableEnum.TextSerialize.WithBinding", "[UAF][unit][MustPass]")
{
	FBindableEnum Src(EUAFTestEnum::ValueB);
	Src.SetBinding(MakeTestBinding());
	FBindableEnum Dst = TextRoundTrip(Src);
	CHECK(Dst.GetConstantValue() == static_cast<int32>(EUAFTestEnum::ValueB));
	CHECK(Dst.GetEnumClass() == StaticEnum<EUAFTestEnum>());
	CHECK(Dst.HasBinding());
}

TEST_CASE("FBindableStruct.TextSerialize.WithBinding", "[UAF][unit][MustPass]")
{
	FBindableStruct Src(FUAFTestVars::StaticStruct());
	Src.SetBinding(MakeTestBinding());
	FBindableStruct Dst = TextRoundTrip(Src);
	CHECK(Dst.GetStructClass() == FUAFTestVars::StaticStruct());
	CHECK(Dst.HasBinding());
}

TEST_CASE("FBindableObject.TextSerialize.WithBinding", "[UAF][unit][MustPass]")
{
	FBindableObject Src(UObject::StaticClass());
	Src.SetBinding(MakeTestBinding());
	FBindableObject Dst = TextRoundTrip(Src);
	CHECK(Dst.GetObjectClass() == UObject::StaticClass());
	CHECK(Dst.HasBinding());
}

// ===========================================================================
// Editor-style set/clear binding tests (mimics SetBindingViaText flow)
// ===========================================================================

TEST_CASE("FBindableBool.EditorSetBinding", "[UAF][unit][MustPass]")
{
	FBindableBool Src(true);
	FBindableBool Dst = EditorStyleSetBinding(Src, MakeTestBinding());
	CHECK(Dst.GetConstantValue() == true);
	CHECK(Dst.HasBinding());
}

TEST_CASE("FBindableFloat.EditorSetBinding", "[UAF][unit][MustPass]")
{
	FBindableFloat Src(42.5f);
	FBindableFloat Dst = EditorStyleSetBinding(Src, MakeTestBinding());
	CHECK(Dst.GetConstantValue() == 42.5f);
	CHECK(Dst.HasBinding());
}

TEST_CASE("FBindableDouble.EditorSetBinding", "[UAF][unit][MustPass]")
{
	FBindableDouble Src(99.9);
	FBindableDouble Dst = EditorStyleSetBinding(Src, MakeTestBinding());
	CHECK(Dst.GetConstantValue() == 99.9);
	CHECK(Dst.HasBinding());
}

TEST_CASE("FBindableInt32.EditorSetBinding", "[UAF][unit][MustPass]")
{
	FBindableInt32 Src(123);
	FBindableInt32 Dst = EditorStyleSetBinding(Src, MakeTestBinding());
	CHECK(Dst.GetConstantValue() == 123);
	CHECK(Dst.HasBinding());
}

TEST_CASE("FBindableInt64.EditorSetBinding", "[UAF][unit][MustPass]")
{
	FBindableInt64 Src(9876543210LL);
	FBindableInt64 Dst = EditorStyleSetBinding(Src, MakeTestBinding());
	CHECK(Dst.GetConstantValue() == 9876543210LL);
	CHECK(Dst.HasBinding());
}

TEST_CASE("FBindableByte.EditorSetBinding", "[UAF][unit][MustPass]")
{
	FBindableByte Src(42);
	FBindableByte Dst = EditorStyleSetBinding(Src, MakeTestBinding());
	CHECK(Dst.GetConstantValue() == 42);
	CHECK(Dst.HasBinding());
}

TEST_CASE("FBindableName.EditorSetBinding", "[UAF][unit][MustPass]")
{
	FBindableName Src(FName(TEXT("TestName")));
	FBindableName Dst = EditorStyleSetBinding(Src, MakeTestBinding());
	CHECK(Dst.GetConstantValue() == FName(TEXT("TestName")));
	CHECK(Dst.HasBinding());
}

TEST_CASE("FBindableVector.EditorSetBinding", "[UAF][unit][MustPass]")
{
	FBindableVector Src(FVector(1.0, 2.0, 3.0));
	FBindableVector Dst = EditorStyleSetBinding(Src, MakeTestBinding());
	CHECK(Dst.GetConstantValue() == FVector(1.0, 2.0, 3.0));
	CHECK(Dst.HasBinding());
}

TEST_CASE("FBindableQuat.EditorSetBinding", "[UAF][unit][MustPass]")
{
	FBindableQuat Src(FQuat(1.0, 2.0, 3.0, 4.0));
	FBindableQuat Dst = EditorStyleSetBinding(Src, MakeTestBinding());
	CHECK(Dst.GetConstantValue().X == 1.0);
	CHECK(Dst.GetConstantValue().W == 4.0);
	CHECK(Dst.HasBinding());
}

TEST_CASE("FBindableTransform.EditorSetBinding", "[UAF][unit][MustPass]")
{
	FBindableTransform Src(FTransform(FQuat(1.0, 2.0, 3.0, 4.0), FVector(5.0, 6.0, 7.0)));
	FBindableTransform Dst = EditorStyleSetBinding(Src, MakeTestBinding());
	CHECK(Dst.GetConstantValue().GetTranslation().X == 5.0);
	CHECK(Dst.HasBinding());
}

TEST_CASE("FBindableEnum.EditorSetBinding", "[UAF][unit][MustPass]")
{
	FBindableEnum Src(EUAFTestEnum::ValueB);
	FBindableEnum Dst = EditorStyleSetBinding(Src, MakeTestBinding());
	CHECK(Dst.GetConstantValue() == static_cast<int32>(EUAFTestEnum::ValueB));
	CHECK(Dst.GetEnumClass() == StaticEnum<EUAFTestEnum>());
	CHECK(Dst.HasBinding());
}

TEST_CASE("FBindableStruct.EditorSetBinding", "[UAF][unit][MustPass]")
{
	FBindableStruct Src(FUAFTestVars::StaticStruct());
	FBindableStruct Dst = EditorStyleSetBinding(Src, MakeTestBinding());
	CHECK(Dst.GetStructClass() == FUAFTestVars::StaticStruct());
	CHECK(Dst.HasBinding());
}

// ===========================================================================
// Self-comparison export tests (simulates FRigVMMemoryStorageStruct::GetDataAsString
// which passes Data as both value and default)
// ===========================================================================

template<typename BindableType>
BindableType SelfComparisonRoundTrip(const BindableType& Source)
{
	// Simulate GetDataAsString(Data, Data): export with self as default
	FString Text;
	Source.ExportTextItem(Text, Source, nullptr, 0, nullptr);

	// Verify the text is well-formed (no leading comma after opening paren)
	CHECK_FALSE(Text.StartsWith(TEXT("(,")));

	// Import into a fresh instance
	BindableType Dest;
	const TCHAR* Buffer = *Text;
	const bool bImported = Dest.ImportTextItem(Buffer, 0, nullptr, nullptr);
	REQUIRE(bImported);
	return Dest;
}

TEST_CASE("FBindableBool.SelfComparisonExport.WithBinding", "[UAF][unit][MustPass]")
{
	FBindableBool Src(true);
	Src.SetBinding(MakeTestBinding());
	FBindableBool Dst = SelfComparisonRoundTrip(Src);
	CHECK(Dst.GetConstantValue() == true);
	CHECK(Dst.HasBinding());
}

TEST_CASE("FBindableBool.SelfComparisonExport.NoBinding", "[UAF][unit][MustPass]")
{
	FBindableBool Src(true);
	FBindableBool Dst = SelfComparisonRoundTrip(Src);
	CHECK(Dst.GetConstantValue() == true);
	CHECK_FALSE(Dst.HasBinding());
}

TEST_CASE("FBindableFloat.SelfComparisonExport.WithBinding", "[UAF][unit][MustPass]")
{
	FBindableFloat Src(42.5f);
	Src.SetBinding(MakeTestBinding());
	FBindableFloat Dst = SelfComparisonRoundTrip(Src);
	CHECK(Dst.GetConstantValue() == 42.5f);
	CHECK(Dst.HasBinding());
}

TEST_CASE("FBindableVector.SelfComparisonExport.WithBinding", "[UAF][unit][MustPass]")
{
	FBindableVector Src(FVector(1.0, 2.0, 3.0));
	Src.SetBinding(MakeTestBinding());
	FBindableVector Dst = SelfComparisonRoundTrip(Src);
	CHECK(Dst.GetConstantValue() == FVector(1.0, 2.0, 3.0));
	CHECK(Dst.HasBinding());
}

TEST_CASE("FBindableTransform.SelfComparisonExport.WithBinding", "[UAF][unit][MustPass]")
{
	FBindableTransform Src(FTransform(FQuat::Identity, FVector(1.0, 2.0, 3.0)));
	Src.SetBinding(MakeTestBinding());
	FBindableTransform Dst = SelfComparisonRoundTrip(Src);
	CHECK(Dst.GetConstantValue().GetTranslation() == FVector(1.0, 2.0, 3.0));
	CHECK(Dst.HasBinding());
}

TEST_CASE("FBindableEnum.SelfComparisonExport.WithBinding", "[UAF][unit][MustPass]")
{
	FBindableEnum Src(EUAFTestEnum::ValueB);
	Src.SetBinding(MakeTestBinding());
	FBindableEnum Dst = SelfComparisonRoundTrip(Src);
	CHECK(Dst.GetConstantValue() == static_cast<int32>(EUAFTestEnum::ValueB));
	CHECK(Dst.GetEnumClass() == StaticEnum<EUAFTestEnum>());
	CHECK(Dst.HasBinding());
}

// Editor-style clear binding
TEST_CASE("FBindableBool.EditorClearBinding", "[UAF][unit][MustPass]")
{
	FBindableBool Src(true);
	Src.SetBinding(MakeTestBinding());
	REQUIRE(Src.HasBinding());
	FBindableBool Dst = EditorStyleClearBinding(Src);
	CHECK(Dst.GetConstantValue() == true);
	CHECK_FALSE(Dst.HasBinding());
}

TEST_CASE("FBindableVector.EditorClearBinding", "[UAF][unit][MustPass]")
{
	FBindableVector Src(FVector(1.0, 2.0, 3.0));
	Src.SetBinding(MakeTestBinding());
	REQUIRE(Src.HasBinding());
	FBindableVector Dst = EditorStyleClearBinding(Src);
	CHECK(Dst.GetConstantValue() == FVector(1.0, 2.0, 3.0));
	CHECK_FALSE(Dst.HasBinding());
}

TEST_CASE("FBindableTransform.EditorClearBinding", "[UAF][unit][MustPass]")
{
	FBindableTransform Src(FTransform(FQuat::Identity, FVector(1.0, 2.0, 3.0)));
	Src.SetBinding(MakeTestBinding());
	REQUIRE(Src.HasBinding());
	FBindableTransform Dst = EditorStyleClearBinding(Src);
	CHECK(Dst.GetConstantValue().GetTranslation() == FVector(1.0, 2.0, 3.0));
	CHECK_FALSE(Dst.HasBinding());
}

// ===========================================================================
// FBindableEnum zero-value tests — validates that enum values whose underlying
// int is 0 survive text serialization. Previously ConstantValue=0 was omitted
// because it matched the default int32 value; the round-trip result would still
// be 0 "by accident" (it is the default), so we directly assert that the
// exported text contains the ConstantValue field.
// ===========================================================================

TEST_CASE("FBindableEnum.TextSerialize.ZeroValueIsExported", "[UAF][unit][MustPass]")
{
	FBindableEnum Src(EUAFTestEnum::ValueA); // ValueA == 0

	FString Text;
	FBindableEnum Default;
	Src.ExportTextItem(Text, Default, nullptr, 0, nullptr);

	// The exported text must explicitly contain ConstantValue=0, not omit it
	CHECK(Text.Contains(TEXT("ConstantValue=0")));
	CHECK(Text.Contains(TEXT("EnumClass=")));

	// Verify round-trip import also works
	FBindableEnum Dst;
	const TCHAR* Buffer = *Text;
	REQUIRE(Dst.ImportTextItem(Buffer, 0, nullptr, nullptr));
	CHECK(Dst.GetConstantValue() == static_cast<int32>(EUAFTestEnum::ValueA));
	CHECK(Dst.GetEnumClass() == StaticEnum<EUAFTestEnum>());
}

TEST_CASE("FBindableEnum.TextSerialize.ZeroValueOverwritesNonZero", "[UAF][unit][MustPass]")
{
	// Export ValueA (== 0), then import into a struct that already has ValueB (== 1).
	// If ConstantValue=0 was omitted from the export, the import would leave
	// the destination at ValueB — proving the bug.
	FBindableEnum Src(EUAFTestEnum::ValueA);

	FString Text;
	FBindableEnum Default;
	Src.ExportTextItem(Text, Default, nullptr, 0, nullptr);

	FBindableEnum Dst(EUAFTestEnum::ValueB);
	const TCHAR* Buffer = *Text;
	REQUIRE(Dst.ImportTextItem(Buffer, 0, nullptr, nullptr));
	CHECK(Dst.GetConstantValue() == static_cast<int32>(EUAFTestEnum::ValueA));
	CHECK(Dst.GetEnumClass() == StaticEnum<EUAFTestEnum>());
}

TEST_CASE("FBindableEnum.SelfComparisonExport.ZeroValueIsExported", "[UAF][unit][MustPass]")
{
	// The RigVM path passes Data as both value and default
	FBindableEnum Src(EUAFTestEnum::ValueA); // ValueA == 0

	FString Text;
	Src.ExportTextItem(Text, Src, nullptr, 0, nullptr);

	CHECK_FALSE(Text.StartsWith(TEXT("(,")));
	CHECK(Text.Contains(TEXT("ConstantValue=0")));
	CHECK(Text.Contains(TEXT("EnumClass=")));

	FBindableEnum Dst;
	const TCHAR* Buffer = *Text;
	REQUIRE(Dst.ImportTextItem(Buffer, 0, nullptr, nullptr));
	CHECK(Dst.GetConstantValue() == static_cast<int32>(EUAFTestEnum::ValueA));
	CHECK(Dst.GetEnumClass() == StaticEnum<EUAFTestEnum>());
}

// ===========================================================================
// Sibling binding isolation tests — when multiple FBindableXxx members are
// inside the same parent struct, ImportTextItem receives the full remaining
// buffer. A member MUST NOT pick up __Binding from a later sibling.
// ===========================================================================

TEST_CASE("FBindableBool.ImportTextItem.DoesNotPickUpSiblingBinding", "[UAF][unit][MustPass]")
{
	// Simulate the buffer that UScriptStruct::ImportText passes when importing
	// BoolVal inside a parent struct where a LATER sibling (Int64Val) has a binding:
	//   (BoolVal=(ConstantValue=True),Int64Val=(ConstantValue=0,__Binding=(...)))
	// When importing BoolVal, the buffer starts at "(ConstantValue=True),Int64Val=..."
	const FString Buffer = TEXT("(ConstantValue=True),Int64Val=(ConstantValue=0,__Binding=(SourceType=Variable,SourceVariable=(Name=\"FloatVal\")))");
	const TCHAR* Ptr = *Buffer;

	FBindableBool Dst;
	const bool bImported = Dst.ImportTextItem(Ptr, 0, nullptr, nullptr);
	REQUIRE(bImported);
	CHECK(Dst.GetConstantValue() == true);
	CHECK_FALSE(Dst.HasBinding()); // Must NOT pick up Int64Val's __Binding
}

TEST_CASE("FBindableFloat.ImportTextItem.DoesNotPickUpSiblingBinding", "[UAF][unit][MustPass]")
{
	const FString Buffer = TEXT("(ConstantValue=42.500000),NextProp=(ConstantValue=0,__Binding=(SourceType=Variable,SourceVariable=(Name=\"FloatVal\")))");
	const TCHAR* Ptr = *Buffer;

	FBindableFloat Dst;
	const bool bImported = Dst.ImportTextItem(Ptr, 0, nullptr, nullptr);
	REQUIRE(bImported);
	CHECK(Dst.GetConstantValue() == 42.5f);
	CHECK_FALSE(Dst.HasBinding());
}

TEST_CASE("FBindableInt64.ImportTextItem.PicksUpOwnBinding", "[UAF][unit][MustPass]")
{
	// The member that DOES have the binding should still import it correctly
	const FString Buffer = TEXT("(ConstantValue=99,__Binding=(SourceType=Variable,SourceVariable=(Name=\"FloatVal\")))");
	const TCHAR* Ptr = *Buffer;

	FBindableInt64 Dst;
	const bool bImported = Dst.ImportTextItem(Ptr, 0, nullptr, nullptr);
	REQUIRE(bImported);
	CHECK(Dst.GetConstantValue() == 99);
	CHECK(Dst.HasBinding());
}

TEST_CASE("FBindableVector.ImportTextItem.DoesNotPickUpSiblingBinding", "[UAF][unit][MustPass]")
{
	// FVector has nested parens in its ConstantValue — make sure the struct boundary detection handles this
	const FString Buffer = TEXT("(ConstantValue=(X=1.000000,Y=2.000000,Z=3.000000)),Sibling=(ConstantValue=0,__Binding=(SourceType=Variable,SourceVariable=(Name=\"FloatVal\")))");
	const TCHAR* Ptr = *Buffer;

	FBindableVector Dst;
	const bool bImported = Dst.ImportTextItem(Ptr, 0, nullptr, nullptr);
	REQUIRE(bImported);
	CHECK(Dst.GetConstantValue() == FVector(1.0, 2.0, 3.0));
	CHECK_FALSE(Dst.HasBinding());
}

TEST_CASE("FBindableTransform.ImportTextItem.DoesNotPickUpSiblingBinding", "[UAF][unit][MustPass]")
{
	// FTransform has deeply nested parens -- verify struct boundary detection handles it
	const FString Buffer = TEXT("(ConstantValue=(Rotation=(X=0.000000,Y=0.000000,Z=0.000000,W=1.000000),Translation=(X=1.000000,Y=2.000000,Z=3.000000),Scale3D=(X=1.000000,Y=1.000000,Z=1.000000))),Sibling=(ConstantValue=0,__Binding=(SourceType=Variable,SourceVariable=(Name=\"FloatVal\")))");
	const TCHAR* Ptr = *Buffer;

	FBindableTransform Dst;
	const bool bImported = Dst.ImportTextItem(Ptr, 0, nullptr, nullptr);
	REQUIRE(bImported);
	CHECK(Dst.GetConstantValue().GetTranslation() == FVector(1.0, 2.0, 3.0));
	CHECK_FALSE(Dst.HasBinding());
}

// ===========================================================================
// Binding-clear change detection tests -- verifies that the RigVM controller's
// SetPinDefaultValue text comparison (RigVMController.cpp) can detect when a
// binding is set or cleared. This is the bug that the always-emit __Binding=()
// behavior in ExportBindingText was introduced to fix.
// ===========================================================================

TEST_CASE("FBindableFloat.TextSerialize.BindingClearProducesDifferentText", "[UAF][unit][MustPass]")
{
	FBindableFloat Default;

	// 1. Export with no binding
	FBindableFloat Unbound(42.5f);
	FString UnboundText;
	Unbound.ExportTextItem(UnboundText, Default, nullptr, 0, nullptr);

	// 2. Set a binding and export
	FBindableFloat Bound(42.5f);
	Bound.SetBinding(MakeTestBinding());
	FString BoundText;
	Bound.ExportTextItem(BoundText, Default, nullptr, 0, nullptr);

	// 3. Clear the binding and export
	FBindableFloat Cleared(42.5f);
	Cleared.SetBinding(MakeTestBinding());
	Cleared.ClearBinding();
	FString ClearedText;
	Cleared.ExportTextItem(ClearedText, Default, nullptr, 0, nullptr);

	// Bound text differs from unbound (binding is visible in exported text)
	CHECK(BoundText != UnboundText);
	// Cleared text differs from bound (clearing is detectable by text comparison)
	CHECK(ClearedText != BoundText);
	// Cleared text contains the empty binding marker
	CHECK(ClearedText.Contains(TEXT("__Binding=()")));
	// Unbound also has the empty marker (always-emit behavior)
	CHECK(UnboundText.Contains(TEXT("__Binding=()")));
}

TEST_CASE("FBindableFloat.TextSerialize.HashStability", "[UAF][unit][MustPass]")
{
	FBindableFloat Src(42.5f);
	FBindableFloat Default;

	FString Text1, Text2;
	Src.ExportTextItem(Text1, Default, nullptr, 0, nullptr);
	Src.ExportTextItem(Text2, Default, nullptr, 0, nullptr);

	CHECK(GetTypeHash(Text1) == GetTypeHash(Text2));
}
