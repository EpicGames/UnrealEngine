// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "TestHarness.h"

#include "AnimNextFunctionReference.h"
#include "RigVMCore/RigVMGraphFunctionDefinition.h"
#include "BindableValue/UAFBindableTypes.h"
#include "BindableValue/UAFPropertyBinding.h"
#include "UAFAssetInstance.h"
#include "Variables/AnimNextVariableReference.h"
#include "Variables/UAFInstanceVariableData.h"
#include "UAFTestVars.h"

#include <catch2/catch_test_macros.hpp>
#include <catch2/catch_approx.hpp>
#include <cmath>
#include <limits>

using Catch::Approx;

// ---------------------------------------------------------------------------
// Test fixture
// ---------------------------------------------------------------------------

namespace UE::UAF::Tests
{
	struct FUAFBindingTestFixture
	{
		static void InitFromStruct(FUAFAssetInstance& InInstance, const UScriptStruct* InStruct)
		{
			InInstance.Variables.AddVariablesContainerForStruct(InStruct, InInstance, nullptr);
			InInstance.Variables.RebuildNameMaps();
		}
	};
} // namespace UE::UAF::Tests

// ---------------------------------------------------------------------------
// Helpers
// ---------------------------------------------------------------------------

namespace
{
	using FFixture = UE::UAF::Tests::FUAFBindingTestFixture;

	TPair<FUAFAssetInstance, FAnimNextVariableReference>
	MakeOwnerAndRef(FName InPropertyName)
	{
		FUAFAssetInstance Owner;
		FFixture::InitFromStruct(Owner, FUAFTestVars::StaticStruct());

		const FProperty* Prop = FUAFTestVars::StaticStruct()->FindPropertyByName(InPropertyName);
		REQUIRE(Prop != nullptr);

		FAnimNextVariableReference Ref =
			FAnimNextVariableReference::FromProperty(Prop, FUAFTestVars::StaticStruct());
		return { MoveTemp(Owner), MoveTemp(Ref) };
	}

	/** Creates a FUAFPropertyBinding with Variable source type. */
	FUAFPropertyBinding MakeVariablePropertyBinding(const FAnimNextVariableReference& SourceVariable)
	{
		FUAFPropertyBinding Binding;
		Binding.SourceType     = EUAFBindingSourceType::Variable;
		Binding.SourceVariable = SourceVariable;
		return Binding;
	}

	/** Creates a FUAFPropertyBinding with SubProperty source type. */
	FUAFPropertyBinding MakeSubPropertyBinding(const FAnimNextVariableReference& SourceVariable, FStringView SubPath)
	{
		FPropertyBindingPath Path;
		const bool bParsed = Path.FromString(SubPath);
		REQUIRE(bParsed);

		FUAFPropertyBinding Binding;
		Binding.SourceType      = EUAFBindingSourceType::SubProperty;
		Binding.SourceVariable  = SourceVariable;
		Binding.SubPropertyPath = MoveTemp(Path);
		return Binding;
	}

	/** Creates a FUAFPropertyBinding with Function source type using a dummy GUID. */
	FUAFPropertyBinding MakeFunctionPropertyBinding()
	{
		FUAFPropertyBinding Binding;
		Binding.SourceType = EUAFBindingSourceType::Function;
		FRigVMGraphFunctionHeader DummyHeader;
		DummyHeader.Name = TEXT("TestFunction");
		DummyHeader.Variant.Guid = FGuid::NewGuid();
		Binding.SourceFunction = FAnimNextFunctionReference::FromHeader(DummyHeader, nullptr);
		return Binding;
	}

} // anonymous namespace

// ---------------------------------------------------------------------------
// FBindableBool
// ---------------------------------------------------------------------------

TEST_CASE("FBindableBool.NoBindingReturnsFalse", "[UAF][unit][MustPass]")
{
	FBindableBool Field;
	Field.SetConstantValue(false);
	REQUIRE_FALSE(Field.HasBinding());

	FUAFAssetInstance Empty;
	REQUIRE(Field.GetValue(&Empty) == false);
}

TEST_CASE("FBindableBool.NoBindingReturnsValue", "[UAF][unit][MustPass]")
{
	FBindableBool Field;
	Field.SetConstantValue(true);
	REQUIRE_FALSE(Field.HasBinding());

	FUAFAssetInstance Empty;
	REQUIRE(Field.GetValue(&Empty) == true);
}

TEST_CASE("FBindableBool.VariableBindingResolvesTrue", "[UAF][unit][MustPass]")
{
	auto [Owner, Ref] = MakeOwnerAndRef("bBool");
	REQUIRE(Owner.SetVariable(Ref, true) == EPropertyBagResult::Success);

	FBindableBool Field;
	Field.SetBinding(MakeVariablePropertyBinding(Ref));

	REQUIRE(Field.HasBinding());
	REQUIRE(Field.GetValue(&Owner) == true);
}

TEST_CASE("FBindableBool.VariableBindingReflectsUpdatedValue", "[UAF][unit][MustPass]")
{
	auto [Owner, Ref] = MakeOwnerAndRef("bBool");

	FBindableBool Field;
	Field.SetConstantValue(true);
	Field.SetBinding(MakeVariablePropertyBinding(Ref));

	REQUIRE(Owner.SetVariable(Ref, false) == EPropertyBagResult::Success);
	REQUIRE(Field.GetValue(&Owner) == false);

	REQUIRE(Owner.SetVariable(Ref, true) == EPropertyBagResult::Success);
	REQUIRE(Field.GetValue(&Owner) == true);
}

TEST_CASE("FBindableBool.FailedResolutionFallsBackToValue", "[UAF][unit][MustPass]")
{
	// Binding with an empty (IsNone) source variable — should fall back to ConstantValue.
	FBindableBool Field;
	Field.SetConstantValue(true);

	FUAFPropertyBinding EmptyBinding;
	EmptyBinding.SourceType = EUAFBindingSourceType::Variable;
	// SourceVariable intentionally default-constructed (IsNone() == true)
	Field.SetBinding(MoveTemp(EmptyBinding));

	REQUIRE(Field.HasBinding());

	FUAFAssetInstance Empty;
	REQUIRE(Field.GetValue(&Empty) == true); // falls back to ConstantValue
}

// ---------------------------------------------------------------------------
// FBindableFloat
// ---------------------------------------------------------------------------

TEST_CASE("FBindableFloat.NoBindingReturnsValue", "[UAF][unit][MustPass]")
{
	FBindableFloat Field;
	Field.SetConstantValue(3.14f);

	FUAFAssetInstance Empty;
	REQUIRE(Field.GetValue(&Empty) == 3.14f);
}

TEST_CASE("FBindableFloat.VariableBindingResolvesValue", "[UAF][unit][MustPass]")
{
	auto [Owner, Ref] = MakeOwnerAndRef("FloatVal");
	REQUIRE(Owner.SetVariable(Ref, 1.5f) == EPropertyBagResult::Success);

	FBindableFloat Field;
	Field.SetBinding(MakeVariablePropertyBinding(Ref));

	REQUIRE(Field.GetValue(&Owner) == 1.5f);
}

TEST_CASE("FBindableFloat.SubPropertyVectorXResolvesToFloat", "[UAF][unit][MustPass]")
{
	auto [Owner, VectorRef] = MakeOwnerAndRef("VectorVar");
	const FVector Source(1.5, 2.5, 3.5);
	REQUIRE(Owner.SetVariable(VectorRef, Source) == EPropertyBagResult::Success);

	FBindableFloat Field;
	Field.SetBinding(MakeSubPropertyBinding(VectorRef, TEXT("X")));

	REQUIRE(Field.GetValue(&Owner) == static_cast<float>(Source.X));
}

TEST_CASE("FBindableFloat.SubPropertyVectorYResolvesToFloat", "[UAF][unit][MustPass]")
{
	auto [Owner, VectorRef] = MakeOwnerAndRef("VectorVar");
	const FVector Source(1.5, 2.5, 3.5);
	REQUIRE(Owner.SetVariable(VectorRef, Source) == EPropertyBagResult::Success);

	FBindableFloat Field;
	Field.SetBinding(MakeSubPropertyBinding(VectorRef, TEXT("Y")));

	REQUIRE(Field.GetValue(&Owner) == static_cast<float>(Source.Y));
}

TEST_CASE("FBindableFloat.SubPropertyVectorZResolvesToFloat", "[UAF][unit][MustPass]")
{
	auto [Owner, VectorRef] = MakeOwnerAndRef("VectorVar");
	const FVector Source(1.5, 2.5, 3.5);
	REQUIRE(Owner.SetVariable(VectorRef, Source) == EPropertyBagResult::Success);

	FBindableFloat Field;
	Field.SetBinding(MakeSubPropertyBinding(VectorRef, TEXT("Z")));

	REQUIRE(Field.GetValue(&Owner) == static_cast<float>(Source.Z));
}

TEST_CASE("FBindableFloat.SubPropertyQuatWResolvesToFloat", "[UAF][unit][MustPass]")
{
	auto [Owner, QuatRef] = MakeOwnerAndRef("QuatVar");
	const FQuat Source = FQuat(FVector::UpVector, UE_HALF_PI);
	REQUIRE(Owner.SetVariable(QuatRef, Source) == EPropertyBagResult::Success);

	FBindableFloat Field;
	Field.SetBinding(MakeSubPropertyBinding(QuatRef, TEXT("W")));

	REQUIRE(Field.GetValue(&Owner) == static_cast<float>(Source.W));
}

TEST_CASE("FBindableFloat.TypeMismatchVariableBindingNoCrash", "[UAF][unit][MustPass]")
{
	auto [Owner, Ref] = MakeOwnerAndRef("VectorVar");
	const FVector Expected(1.0, 2.0, 3.0);
	REQUIRE(Owner.SetVariable(Ref, Expected) == EPropertyBagResult::Success);

	// FVector variable → float target: should fall back gracefully, no crash.
	FBindableFloat Field;
	Field.SetConstantValue(99.0f);
	Field.SetBinding(MakeVariablePropertyBinding(Ref));

	const float Result = Field.GetValue(&Owner);
	(void)Result;
}

// ---------------------------------------------------------------------------
// FBindableDouble
// ---------------------------------------------------------------------------

TEST_CASE("FBindableDouble.NoBindingReturnsValue", "[UAF][unit][MustPass]")
{
	FBindableDouble Field;
	Field.SetConstantValue(2.71828);

	FUAFAssetInstance Empty;
	REQUIRE(Field.GetValue(&Empty) == 2.71828);
}

TEST_CASE("FBindableDouble.VariableBindingResolvesValue", "[UAF][unit][MustPass]")
{
	auto [Owner, Ref] = MakeOwnerAndRef("DoubleVal");
	REQUIRE(Owner.SetVariable(Ref, 2.718) == EPropertyBagResult::Success);

	FBindableDouble Field;
	Field.SetBinding(MakeVariablePropertyBinding(Ref));

	REQUIRE(Field.GetValue(&Owner) == 2.718);
}

TEST_CASE("FBindableDouble.SubPropertyVectorXResolvesToDouble", "[UAF][unit][MustPass]")
{
	auto [Owner, VectorRef] = MakeOwnerAndRef("VectorVar");
	const FVector Source(1.23456789, 0.0, 0.0);
	REQUIRE(Owner.SetVariable(VectorRef, Source) == EPropertyBagResult::Success);

	FBindableDouble Field;
	Field.SetBinding(MakeSubPropertyBinding(VectorRef, TEXT("X")));

	REQUIRE(Field.GetValue(&Owner) == Source.X);
}

TEST_CASE("FBindableDouble.SubPropertyQuatWResolvesToDouble", "[UAF][unit][MustPass]")
{
	auto [Owner, QuatRef] = MakeOwnerAndRef("QuatVar");
	const FQuat Source = FQuat(FVector::UpVector, UE_HALF_PI);
	REQUIRE(Owner.SetVariable(QuatRef, Source) == EPropertyBagResult::Success);

	FBindableDouble Field;
	Field.SetBinding(MakeSubPropertyBinding(QuatRef, TEXT("W")));

	REQUIRE(Field.GetValue(&Owner) == Source.W);
}

// ---------------------------------------------------------------------------
// FBindableInt32
// ---------------------------------------------------------------------------

TEST_CASE("FBindableInt32.NoBindingReturnsValue", "[UAF][unit][MustPass]")
{
	FBindableInt32 Field;
	Field.SetConstantValue(42);

	FUAFAssetInstance Empty;
	REQUIRE(Field.GetValue(&Empty) == 42);
}

TEST_CASE("FBindableInt32.VariableBindingResolvesValue", "[UAF][unit][MustPass]")
{
	auto [Owner, Ref] = MakeOwnerAndRef("IntVal");
	REQUIRE(Owner.SetVariable(Ref, 99) == EPropertyBagResult::Success);

	FBindableInt32 Field;
	Field.SetBinding(MakeVariablePropertyBinding(Ref));

	REQUIRE(Field.GetValue(&Owner) == 99);
}

// ---------------------------------------------------------------------------
// FBindableInt64
// ---------------------------------------------------------------------------

TEST_CASE("FBindableInt64.NoBindingReturnsValue", "[UAF][unit][MustPass]")
{
	FBindableInt64 Field;
	Field.SetConstantValue(1234567890123LL);

	FUAFAssetInstance Empty;
	REQUIRE(Field.GetValue(&Empty) == 1234567890123LL);
}

TEST_CASE("FBindableInt64.VariableBindingResolvesValue", "[UAF][unit][MustPass]")
{
	auto [Owner, Ref] = MakeOwnerAndRef("Int64Val");
	REQUIRE(Owner.SetVariable(Ref, (int64)9876543210LL) == EPropertyBagResult::Success);

	FBindableInt64 Field;
	Field.SetBinding(MakeVariablePropertyBinding(Ref));

	REQUIRE(Field.GetValue(&Owner) == (int64)9876543210LL);
}

// ---------------------------------------------------------------------------
// FBindableByte
// ---------------------------------------------------------------------------

TEST_CASE("FBindableByte.NoBindingReturnsValue", "[UAF][unit][MustPass]")
{
	FBindableByte Field;
	Field.SetConstantValue((uint8)255);

	FUAFAssetInstance Empty;
	REQUIRE(Field.GetValue(&Empty) == (uint8)255);
}

TEST_CASE("FBindableByte.VariableBindingResolvesValue", "[UAF][unit][MustPass]")
{
	auto [Owner, Ref] = MakeOwnerAndRef("ByteVal");
	REQUIRE(Owner.SetVariable(Ref, (uint8)42) == EPropertyBagResult::Success);

	FBindableByte Field;
	Field.SetBinding(MakeVariablePropertyBinding(Ref));

	REQUIRE(Field.GetValue(&Owner) == (uint8)42);
}

// ---------------------------------------------------------------------------
// FBindableName
// ---------------------------------------------------------------------------

TEST_CASE("FBindableName.NoBindingReturnsValue", "[UAF][unit][MustPass]")
{
	FBindableName Field;
	Field.SetConstantValue(FName("TestName"));

	FUAFAssetInstance Empty;
	REQUIRE(Field.GetValue(&Empty) == FName("TestName"));
}

TEST_CASE("FBindableName.VariableBindingResolvesValue", "[UAF][unit][MustPass]")
{
	auto [Owner, Ref] = MakeOwnerAndRef("NameVar");
	REQUIRE(Owner.SetVariable(Ref, FName("BoundName")) == EPropertyBagResult::Success);

	FBindableName Field;
	Field.SetBinding(MakeVariablePropertyBinding(Ref));

	REQUIRE(Field.GetValue(&Owner) == FName("BoundName"));
}

// ---------------------------------------------------------------------------
// FBindableEnum
// ---------------------------------------------------------------------------

TEST_CASE("FBindableEnum.NoBindingReturnsValue", "[UAF][unit][MustPass]")
{
	FBindableEnum Field;
	Field.SetConstantValue((int32)EUAFTestEnum::ValueC);
	Field.SetEnumClass(StaticEnum<EUAFTestEnum>());

	FUAFAssetInstance Empty;
	REQUIRE(Field.GetValue(&Empty) == (int32)EUAFTestEnum::ValueC);
}

TEST_CASE("FBindableEnum.VariableBindingResolvesValue", "[UAF][unit][MustPass]")
{
	auto [Owner, Ref] = MakeOwnerAndRef("EnumVar");
	REQUIRE(Owner.SetVariable(Ref, (uint8)EUAFTestEnum::ValueB) == EPropertyBagResult::Success);

	FBindableEnum Field;
	Field.SetConstantValue((int32)EUAFTestEnum::ValueA);
	Field.SetEnumClass(StaticEnum<EUAFTestEnum>());
	Field.SetBinding(MakeVariablePropertyBinding(Ref));

	REQUIRE(Field.GetValue(&Owner) == (int32)EUAFTestEnum::ValueB);
}

TEST_CASE("FBindableEnum.BytePropertyVariableBindingResolvesValue", "[UAF][unit][MustPass]")
{
	auto [Owner, Ref] = MakeOwnerAndRef("EnumAsByteVar");
	REQUIRE(Owner.SetVariable(Ref, (uint8)EUAFTestByteEnum::ByteC) == EPropertyBagResult::Success);

	UEnum* ByteEnumClass = StaticEnum<EUAFTestByteEnum::Type>();

	FBindableEnum Field;
	Field.SetConstantValue((int32)EUAFTestByteEnum::ByteA);
	Field.SetEnumClass(ByteEnumClass);
	Field.SetBinding(MakeVariablePropertyBinding(Ref));

	REQUIRE(Field.GetValue(&Owner) == (int32)EUAFTestByteEnum::ByteC);
}

TEST_CASE("FBindableEnum.VariableBinding_SmallEnum_DoesNotOverread", "[UAF][unit][MustPass]")
{
	// EnumVar is uint8 (1 byte) but FBindableEnum stores int32 (4 bytes).
	// Verify the resolved value only reads the 1-byte source and zero-fills the rest,
	// rather than reading 4 bytes from a 1-byte property (which would overread).
	auto [Owner, Ref] = MakeOwnerAndRef("EnumVar");
	REQUIRE(Owner.SetVariable(Ref, (uint8)EUAFTestEnum::ValueC) == EPropertyBagResult::Success);

	// Pre-fill ConstantValue with a sentinel that would be visible if upper bytes leak through
	FBindableEnum Field;
	Field.SetConstantValue(0x7F7F7F7F);
	Field.SetEnumClass(StaticEnum<EUAFTestEnum>());
	Field.SetBinding(MakeVariablePropertyBinding(Ref));

	const int32 Result = Field.GetValue(&Owner);
	// ValueC == 2.  Upper 3 bytes must be zero, not leftover from ConstantValue.
	REQUIRE(Result == (int32)EUAFTestEnum::ValueC);
	REQUIRE((Result & 0xFFFFFF00) == 0);
}

TEST_CASE("FBindableInt32.SubPropertyBinding_SmallField_DoesNotOverread", "[UAF][unit][MustPass]")
{
	// PackedByteVar has {uint8 Val1, uint8 Val2} adjacent in memory.
	// Binding a FBindableInt32 (4 bytes) to Val1 (1 byte) via SubProperty must NOT
	// read Val2 into the upper bytes of the result.
	auto [Owner, Ref] = MakeOwnerAndRef("PackedByteVar");

	FUAFPackedByteStruct Packed;
	Packed.Val1 = 0x0A;
	Packed.Val2 = 0xFF; // sentinel — must NOT appear in result
	REQUIRE(Owner.SetVariable(Ref, Packed) == EPropertyBagResult::Success);

	FBindableInt32 Field;
	Field.SetConstantValue(0x7F7F7F7F); // sentinel in ConstantValue too
	Field.SetBinding(MakeSubPropertyBinding(Ref, TEXT("Val1")));

	const int32 Result = Field.GetValue(&Owner);
	REQUIRE(Result == 0x0A);
	// Upper bytes must be zero — no overread from Val2 (0xFF) or ConstantValue (0x7F...)
	REQUIRE((Result & 0xFFFFFF00) == 0);
}

TEST_CASE("FBindableByte.SubPropertyBinding_SmallField_DoesNotOverread", "[UAF][unit][MustPass]")
{
	// Same setup but binding FBindableByte (1 byte target) to Val1 (1 byte source).
	// This should be an exact size match — verify Val2 doesn't leak.
	auto [Owner, Ref] = MakeOwnerAndRef("PackedByteVar");

	FUAFPackedByteStruct Packed;
	Packed.Val1 = 42;
	Packed.Val2 = 0xFF; // sentinel
	REQUIRE(Owner.SetVariable(Ref, Packed) == EPropertyBagResult::Success);

	FBindableByte Field;
	Field.SetBinding(MakeSubPropertyBinding(Ref, TEXT("Val1")));

	REQUIRE(Field.GetValue(&Owner) == 42);
}

TEST_CASE("FBindableEnum.SubPropertyBinding_SmallEnum_DoesNotOverread", "[UAF][unit][MustPass]")
{
	// PackedByteVar.EnumVal is uint8 enum (1 byte). FBindableEnum stores int32 (4 bytes).
	// SubProperty binding must not read adjacent bytes.
	auto [Owner, Ref] = MakeOwnerAndRef("PackedByteVar");

	FUAFPackedByteStruct Packed;
	Packed.EnumVal = EUAFTestEnum::ValueB;
	Packed.FloatVal = -1.0f; // adjacent sentinel
	REQUIRE(Owner.SetVariable(Ref, Packed) == EPropertyBagResult::Success);

	FBindableEnum Field;
	Field.SetConstantValue(0x7F7F7F7F);
	Field.SetEnumClass(StaticEnum<EUAFTestEnum>());
	Field.SetBinding(MakeSubPropertyBinding(Ref, TEXT("EnumVal")));

	const int32 Result = Field.GetValue(&Owner);
	REQUIRE(Result == (int32)EUAFTestEnum::ValueB);
	REQUIRE((Result & 0xFFFFFF00) == 0);
}

TEST_CASE("FBindableInt32.VariableBinding_SmallByte_DoesNotOverread", "[UAF][unit][MustPass]")
{
	// ByteVal is uint8 (1 byte). FBindableInt32 stores int32 (4 bytes).
	auto [Owner, Ref] = MakeOwnerAndRef("ByteVal");
	REQUIRE(Owner.SetVariable(Ref, (uint8)0x42) == EPropertyBagResult::Success);

	FBindableInt32 Field;
	Field.SetConstantValue(0x7F7F7F7F);
	Field.SetBinding(MakeVariablePropertyBinding(Ref));

	const int32 Result = Field.GetValue(&Owner);
	// GetVariable handles type conversion — should return 0x42 with clean upper bytes
	REQUIRE(Result == 0x42);
}

TEST_CASE("FBindableDouble.VariableBinding_Float_DoesNotOverread", "[UAF][unit][MustPass]")
{
	// FloatVal is float (4 bytes). FBindableDouble stores double (8 bytes).
	// The resolution should promote float→double cleanly.
	auto [Owner, Ref] = MakeOwnerAndRef("FloatVal");
	REQUIRE(Owner.SetVariable(Ref, 3.14f) == EPropertyBagResult::Success);

	FBindableDouble Field;
	Field.SetConstantValue(std::numeric_limits<double>::quiet_NaN()); // sentinel
	Field.SetBinding(MakeVariablePropertyBinding(Ref));

	const double Result = Field.GetValue(&Owner);
	REQUIRE(Result == Approx(3.14).epsilon(0.001));
	REQUIRE(!std::isnan(Result));
}

TEST_CASE("FBindableDouble.SubPropertyBinding_Float_DoesNotOverread", "[UAF][unit][MustPass]")
{
	// PackedByteVar.FloatVal is float (4 bytes). FBindableDouble stores double (8 bytes).
	auto [Owner, Ref] = MakeOwnerAndRef("PackedByteVar");

	FUAFPackedByteStruct Packed;
	Packed.FloatVal = 2.718f;
	REQUIRE(Owner.SetVariable(Ref, Packed) == EPropertyBagResult::Success);

	FBindableDouble Field;
	Field.SetConstantValue(std::numeric_limits<double>::quiet_NaN()); // sentinel
	Field.SetBinding(MakeSubPropertyBinding(Ref, TEXT("FloatVal")));

	const double Result = Field.GetValue(&Owner);
	REQUIRE(Result == Approx(2.718).epsilon(0.001));
	REQUIRE(!std::isnan(Result));
}

// ---------------------------------------------------------------------------
// FBindableVector
// ---------------------------------------------------------------------------

TEST_CASE("FBindableVector.NoBindingReturnsConstantValue", "[UAF][unit][MustPass]")
{
	FBindableVector Field;
	Field.SetConstantValue(FVector(1.0, 2.0, 3.0));

	REQUIRE(Field.GetValue(nullptr) == FVector(1.0, 2.0, 3.0));
}

TEST_CASE("FBindableVector.VariableBindingResolvesVector", "[UAF][unit][MustPass]")
{
	auto [Owner, VectorRef] = MakeOwnerAndRef("VectorVar");
	const FVector Source(10.0, 20.0, 30.0);
	REQUIRE(Owner.SetVariable(VectorRef, Source) == EPropertyBagResult::Success);

	FBindableVector Field;
	Field.SetBinding(MakeVariablePropertyBinding(VectorRef));

	REQUIRE(Field.GetValue(&Owner) == Source);
}

TEST_CASE("FBindableVector.SubPropertyNestedVecResolvesToVector", "[UAF][unit][MustPass]")
{
	auto [Owner, NestedRef] = MakeOwnerAndRef("NestedVar");
	FUAFNestedTestStruct NestedData;
	NestedData.Vec = FVector(7.0, 8.0, 9.0);
	REQUIRE(Owner.SetVariable(NestedRef, NestedData) == EPropertyBagResult::Success);

	FBindableVector Field;
	Field.SetBinding(MakeSubPropertyBinding(NestedRef, TEXT("Vec")));

	REQUIRE(Field.GetValue(&Owner) == NestedData.Vec);
}

// ---------------------------------------------------------------------------
// FBindableQuat
// ---------------------------------------------------------------------------

TEST_CASE("FBindableQuat.NoBindingReturnsConstantValue", "[UAF][unit][MustPass]")
{
	const FQuat Expected = FQuat(FVector::UpVector, UE_HALF_PI);
	FBindableQuat Field;
	Field.SetConstantValue(Expected);

	REQUIRE(Field.GetValue(nullptr) == Expected);
}

TEST_CASE("FBindableQuat.VariableBindingResolvesQuat", "[UAF][unit][MustPass]")
{
	auto [Owner, QuatRef] = MakeOwnerAndRef("QuatVar");
	const FQuat Source = FQuat(FVector::UpVector, UE_HALF_PI);
	REQUIRE(Owner.SetVariable(QuatRef, Source) == EPropertyBagResult::Success);

	FBindableQuat Field;
	Field.SetBinding(MakeVariablePropertyBinding(QuatRef));

	REQUIRE(Field.GetValue(&Owner) == Source);
}

TEST_CASE("FBindableQuat.SubPropertyNestedQuatResolvesToQuat", "[UAF][unit][MustPass]")
{
	auto [Owner, NestedRef] = MakeOwnerAndRef("NestedVar");
	FUAFNestedTestStruct NestedData;
	NestedData.Quat = FQuat(FVector::ForwardVector, UE_HALF_PI);
	REQUIRE(Owner.SetVariable(NestedRef, NestedData) == EPropertyBagResult::Success);

	FBindableQuat Field;
	Field.SetBinding(MakeSubPropertyBinding(NestedRef, TEXT("Quat")));

	REQUIRE(Field.GetValue(&Owner) == NestedData.Quat);
}

// ---------------------------------------------------------------------------
// FBindableTransform
// ---------------------------------------------------------------------------

TEST_CASE("FBindableTransform.NoBindingReturnsConstantValue", "[UAF][unit][MustPass]")
{
	const FTransform Expected(FQuat(FVector::UpVector, UE_HALF_PI), FVector(1.0, 2.0, 3.0), FVector(1.0, 1.0, 1.0));
	FBindableTransform Field;
	Field.SetConstantValue(Expected);

	REQUIRE(Field.GetValue(nullptr).Equals(Expected));
}

TEST_CASE("FBindableTransform.VariableBindingResolvesTransform", "[UAF][unit][MustPass]")
{
	auto [Owner, TransformRef] = MakeOwnerAndRef("TransformVar");
	const FTransform Source(FQuat(FVector::UpVector, UE_HALF_PI), FVector(10.0, 20.0, 30.0), FVector(1.0, 1.0, 1.0));
	REQUIRE(Owner.SetVariable(TransformRef, Source) == EPropertyBagResult::Success);

	FBindableTransform Field;
	Field.SetBinding(MakeVariablePropertyBinding(TransformRef));

	REQUIRE(Field.GetValue(&Owner).Equals(Source));
}

TEST_CASE("FBindableTransform.SubPropertyNestedTransformResolvesToTransform", "[UAF][unit][MustPass]")
{
	auto [Owner, NestedRef] = MakeOwnerAndRef("NestedVar");
	FUAFNestedTestStruct NestedData;
	NestedData.Transform = FTransform(FQuat(FVector::ForwardVector, UE_HALF_PI), FVector(7.0, 8.0, 9.0));
	REQUIRE(Owner.SetVariable(NestedRef, NestedData) == EPropertyBagResult::Success);

	FBindableTransform Field;
	Field.SetBinding(MakeSubPropertyBinding(NestedRef, TEXT("Transform")));

	REQUIRE(Field.GetValue(&Owner).Equals(NestedData.Transform));
}

// ---------------------------------------------------------------------------
// FBindableStruct
// ---------------------------------------------------------------------------

TEST_CASE("FBindableStruct.NoBindingReturnsConstantValue", "[UAF][unit][MustPass]")
{
	FBindableStruct Field;
	FInstancedStruct Tmp;
	Tmp.InitializeAs<FVector>(FVector(1.0, 2.0, 3.0));
	Field.SetConstantValue(MoveTemp(Tmp));

	FVector Result = FVector::ZeroVector;
	Field.GetValue(nullptr, Result);   // null Instance → falls back to ConstantValue
	REQUIRE(Result == FVector(1.0, 2.0, 3.0));
}

TEST_CASE("FBindableStruct.VariableBindingResolvesVector", "[UAF][unit][MustPass]")
{
	auto [Owner, VectorRef] = MakeOwnerAndRef("VectorVar");
	const FVector Source(4.0, 5.0, 6.0);
	REQUIRE(Owner.SetVariable(VectorRef, Source) == EPropertyBagResult::Success);

	FBindableStruct Field;
	Field.SetBinding(MakeVariablePropertyBinding(VectorRef));

	FVector Result = FVector::ZeroVector;
	Field.GetValue(&Owner, Result);
	REQUIRE(Result == Source);
}

TEST_CASE("FBindableStruct.SubPropertyNestedVecResolvesToVector", "[UAF][unit][MustPass]")
{
	auto [Owner, NestedRef] = MakeOwnerAndRef("NestedVar");
	FUAFNestedTestStruct NestedData;
	NestedData.Vec = FVector(7.0, 8.0, 9.0);
	REQUIRE(Owner.SetVariable(NestedRef, NestedData) == EPropertyBagResult::Success);

	FBindableStruct Field;
	Field.SetBinding(MakeSubPropertyBinding(NestedRef, TEXT("Vec")));

	FVector Result = FVector::ZeroVector;
	Field.GetValue(&Owner, Result);
	REQUIRE(Result == NestedData.Vec);
}

TEST_CASE("FBindableStruct.EmptyConstantValueInitializesOutput", "[UAF][unit][MustPass]")
{
	// Construct with StructClass only — no ConstantValue
	FBindableStruct Field(TBaseStructure<FVector>::Get());

	// Fill OutValue with garbage to detect if it's left unmodified
	FVector OutValue(999.0, 999.0, 999.0);
	Field.GetValue<FVector>(nullptr, OutValue);

	// Should be zero-initialized, not left as garbage
	REQUIRE(OutValue == FVector::ZeroVector);
}

// ---------------------------------------------------------------------------
// FBindableObject
// ---------------------------------------------------------------------------

TEST_CASE("FBindableObject.NoBindingReturnsNull", "[UAF][unit][MustPass]")
{
	FBindableObject Field;
	REQUIRE_FALSE(Field.HasBinding());
	REQUIRE(Field.GetValue(nullptr) == nullptr);
}

TEST_CASE("FBindableObject.NoBindingReturnsConstant", "[UAF][unit][MustPass]")
{
	UObject* TestObj = UObject::StaticClass()->GetDefaultObject();
	FBindableObject Field;
	Field.SetConstantValue(TestObj);

	FUAFAssetInstance Empty;
	REQUIRE(Field.GetValue(&Empty) == TestObj);
}

TEST_CASE("FBindableObject.VariableBindingResolvesObject", "[UAF][unit][MustPass]")
{
	auto [Owner, ObjRef] = MakeOwnerAndRef("ObjectVar");
	UObject* TestObj = UObject::StaticClass()->GetDefaultObject();

	const FAnimNextParamType ObjParamType(EPropertyBagPropertyType::Object,
		EPropertyBagContainerType::None, UObject::StaticClass());
	REQUIRE(Owner.SetVariable(ObjRef, ObjParamType,
		TConstArrayView<uint8>(reinterpret_cast<const uint8*>(&TestObj), sizeof(TObjectPtr<UObject>))) == EPropertyBagResult::Success);

	FBindableObject Field;
	Field.SetObjectClass(UObject::StaticClass());
	Field.SetBinding(MakeVariablePropertyBinding(ObjRef));

	REQUIRE(Field.GetValue(&Owner) == TestObj);
}

// ---------------------------------------------------------------------------
// Function binding fallback tests
// ---------------------------------------------------------------------------
// These tests verify that Function-type bindings gracefully fall back to
// ConstantValue when no RigVM component is available (the normal case in
// unit tests without a fully compiled asset).

TEST_CASE("FBindableFloat.FunctionBindingFailure_ReturnsConstantValue_NotZero", "[UAF][unit][MustPass]")
{
	// When a Function binding has a valid (non-None) SourceFunction but resolution fails
	// (e.g. no RigVM component), the result must be ConstantValue — not zero.
	// This catches the bug where zero-init of Out leaked through on failed resolution.
	FBindableFloat Field;
	Field.SetConstantValue(999.0f);

	// Construct a binding with a valid GUID (non-None) but no matching asset/component.
	FUAFPropertyBinding FuncBinding;
	FuncBinding.SourceType = EUAFBindingSourceType::Function;
	// Manually set FunctionGuid to a non-zero value so IsNone() returns false.
	// Use placement to access the private FunctionGuid field via the struct's UPROPERTY serialization.
	// Simplest approach: use FromHeader with a fake header that has a valid GUID.
	FRigVMGraphFunctionHeader FakeHeader;
	FakeHeader.Name = TEXT("FakeFunction");
	FakeHeader.Variant.Guid = FGuid::NewGuid();
	FuncBinding.SourceFunction = FAnimNextFunctionReference::FromHeader(FakeHeader, nullptr);
	Field.SetBinding(MoveTemp(FuncBinding));

	REQUIRE(Field.HasBinding());

	// Instance without a RigVM component — resolution will fail.
	FUAFAssetInstance Empty;
	// Must return ConstantValue (999.0), NOT zero.
	REQUIRE(Field.GetValue(&Empty) == 999.0f);
}

TEST_CASE("FBindableBool.FunctionBindingWithNoneRefFallsBack", "[UAF][unit][MustPass]")
{
	FBindableBool Field;
	Field.SetConstantValue(true);

	// Function binding with an IsNone reference
	FUAFPropertyBinding EmptyFunctionBinding;
	EmptyFunctionBinding.SourceType = EUAFBindingSourceType::Function;
	// SourceFunction intentionally default-constructed (IsNone() == true)
	Field.SetBinding(MoveTemp(EmptyFunctionBinding));

	REQUIRE(Field.HasBinding());

	FUAFAssetInstance Empty;
	REQUIRE(Field.GetValue(&Empty) == true);
}


TEST_CASE("FBindableBool.FunctionBindingFallsBackWithoutRigVM", "[UAF][unit][MustPass]")
{
	FBindableBool Field;
	Field.SetConstantValue(true);
	Field.SetBinding(MakeFunctionPropertyBinding());

	REQUIRE(Field.HasBinding());

	// Instance without a RigVM component -> falls back to ConstantValue
	FUAFAssetInstance Empty;
	REQUIRE(Field.GetValue(&Empty) == true);
}

TEST_CASE("FBindableFloat.FunctionBindingFallsBackWithoutRigVM", "[UAF][unit][MustPass]")
{
	FBindableFloat Field;
	Field.SetConstantValue(42.0f);
	Field.SetBinding(MakeFunctionPropertyBinding());

	FUAFAssetInstance Empty;
	REQUIRE(Field.GetValue(&Empty) == 42.0f);
}

TEST_CASE("FBindableFloat.FunctionBindingWithNullInstanceFallsBack", "[UAF][unit][MustPass]")
{
	FBindableFloat Field;
	Field.SetConstantValue(7.5f);
	Field.SetBinding(MakeFunctionPropertyBinding());

	// null instance -> falls back to ConstantValue
	REQUIRE(Field.GetValue(nullptr) == 7.5f);
}

TEST_CASE("FBindableEnum.FunctionBindingFallsBackWithoutRigVM", "[UAF][unit][MustPass]")
{
	FBindableEnum Field;
	Field.SetConstantValue((int32)EUAFTestEnum::ValueB);
	Field.SetEnumClass(StaticEnum<EUAFTestEnum>());
	Field.SetBinding(MakeFunctionPropertyBinding());

	FUAFAssetInstance Empty;
	REQUIRE(Field.GetValue(&Empty) == (int32)EUAFTestEnum::ValueB);
}

TEST_CASE("FBindableStruct.FunctionBindingFallsBackWithoutRigVM", "[UAF][unit][MustPass]")
{
	FBindableStruct Field;
	FInstancedStruct Tmp;
	Tmp.InitializeAs<FVector>(FVector(1.0, 2.0, 3.0));
	Field.SetConstantValue(MoveTemp(Tmp));
	Field.SetBinding(MakeFunctionPropertyBinding());

	// Without a RigVM component, falls back to ConstantValue
	FVector Result = FVector::ZeroVector;
	FUAFAssetInstance Empty;
	Field.GetValue<FVector>(&Empty, Result);
	REQUIRE(Result == FVector(1.0, 2.0, 3.0));
}

TEST_CASE("FBindableObject.FunctionBindingFallsBackWithoutRigVM", "[UAF][unit][MustPass]")
{
	FBindableObject Field;
	UObject* TestObj = UObject::StaticClass()->GetDefaultObject();
	Field.SetConstantValue(TestObj);
	Field.SetBinding(MakeFunctionPropertyBinding());

	FUAFAssetInstance Empty;
	REQUIRE(Field.GetValue(&Empty) == TestObj);
}

