// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "TestCommon/Initialization.h"
#include "TestRunner.h"

class FUAFTestsModule : public IModuleInterface
{
};
IMPLEMENT_MODULE(FUAFTestsModule, UAFTests)

namespace
{
	void SetupUAFTests()
	{
		InitAll(/*bAllowLogging=*/false, /*bMultithreaded=*/false);
	}
	void TeardownUAFTests()
	{
		CleanupAll();
	}
	struct FUAFTestsGlobalSetup
	{
		FUAFTestsGlobalSetup()
		{
			FTestDelegates::GetGlobalSetup().BindStatic(&SetupUAFTests);
			FTestDelegates::GetGlobalTeardown().BindStatic(&TeardownUAFTests);
		}
	} GUAFTestsGlobalSetup;
}
