// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedPlatforms(UnrealPlatformClass.All)]
public class UAFAllTestsTarget : TestTargetRules
{
	public UAFAllTestsTarget(TargetInfo Target) : base(Target)
	{
		bUsePlatformFileStub = true;
		bMockEngineDefaults = true;

		bCompileAgainstEngine = true;
		bCompileAgainstApplicationCore = true;

		bTestsRequireEngine = true;

		bWarningsAsErrors = true;

		GlobalDefinitions.Add("CATCH_CONFIG_ENABLE_BENCHMARKING=1");

		LaunchModuleName = "UAFTests";

		// Enable all UAF sub-plugins so UBT can discover their test modules.
		EnablePlugins.AddRange(new string[]
		{
			"UAFAnimNode",
			"UAFAnimGraph",
			"UAFChooser",
			"UAFControlRig",
			"UAFLayering",
			"UAFMass",
			"UAFMirroring",
			"UAFPoseSearch",
			"UAFStateTree",
			"UAFWarping",
		});

		ExtraModuleNames.AddRange(new string[]
		{
			"UAFAnimNodeTests",
			"UAFAnimGraphTests",
			"UAFChooserTests",
			"UAFControlRigTests",
			"UAFLayeringTests",
			"UAFMassTests",
			"UAFMirroringTests",
			"UAFPoseSearchTests",
			"UAFStateTreeTests",
			"UAFWarpingTests",
		});
	}
}
