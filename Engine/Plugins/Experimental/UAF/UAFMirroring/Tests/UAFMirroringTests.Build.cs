// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class UAFMirroringTests : TestModuleRules
{
	public UAFMirroringTests(ReadOnlyTargetRules Target) : base(Target, true)
	{
		bAllowUETypesInNamespaces = true;
		
		PrivateIncludePaths.AddRange(
			// Any private include paths
			new string[] {
			}	
		);
		
		PrivateDependencyModuleNames.AddRange(
			// Any private dependencies to link against
			new string[] {
				"UAFMirroring"
			}			
		);
	}
}