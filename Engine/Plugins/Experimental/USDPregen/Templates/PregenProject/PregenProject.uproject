{
	"FileVersion": 3,
	"EngineAssociation": "{5532D745-4181-BACF-3060-3EAC0C61F5D3}",
	"Category": "",
	"Description": "",
	"Plugins": [
		{
			"Name": "USDPregen",
			"Enabled": true
		},
		{
			"Name": "InterchangeOpenUSD",
			"Enabled": true,
			"SupportedTargetPlatforms": [
				"Win64",
				"Linux",
				"Mac"
			]
		},
		{
			"Name": "PregenContent",
			"Enabled": true
		}
	]
}