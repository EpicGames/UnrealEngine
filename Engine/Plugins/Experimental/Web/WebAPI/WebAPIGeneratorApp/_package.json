{
  "name": "web-api-generator-server",
  "version": "0.0.1",
  "description": "WebAPIGeneratorApp initialised with create-express-typescript-application.",
  "main": "src/index.ts",
  "scripts": {
    "build": "tsc -p tsconfig.json",
    "start": "node ./build/index.js",
    "dev": "ts-node ./src/index.ts",
    "dev:nodemon": "tsc && nodemon -w src -w templates -e ts,json,liquid -x ts-node ./src/index.ts",
    "find-deadcode": "ts-prune"
  },
  "keywords": ["express", "typescript"],
  "devDependencies": {
    "@types/express": "^4.17.13",
    "@types/node": "^14.17.14",
    "body-parser": "^1.19.0",
    "express": "^4.17.1",
    "fs-extra": "^9.0.1",
    "lodash": "^4.17.20",
    "nodemon": "^2.0.12",
    "superagent": "^6.0.0",
    "ts-node": "^9.1.1",
    "typescript": "^4.4.2",
    "ws": "^7.3.1"
  },
  "dependencies": {
    "@overnightjs/core": "^1.7.6",
    "@types/express": "^4.17.7",
    "@types/fs-extra": "^9.0.1",
    "@types/lodash": "^4.14.160",
    "@types/node": "^14.6.0",
    "@types/superagent": "4.1.10",
    "@types/ws": "^7.2.6",  
    "express": "^4.17.1",
    "http-status": "^1.5.0",
    "liquidjs": "^9.25.1",
    "typescript": "^4.0.2"    
  }
}
