const fs = require('fs-extra');
const path = require('path');
const exec = require('child_process').exec;
const util = require('util');
const package = require('../package.json');

const execute = util.promisify(exec);

async function build() {
  try {
    const root = path.resolve(__dirname, '..');

    await Promise.all([
      execute('npm run build', { cwd: root })
    ])

    await fs.writeJSON(path.join(root, 'public', 'version.json'), { date: new Date(), version: package.version });

  } catch (err) {
    let message = err.stdout;
    if (!message)
      message = err.message;
    if (!message)
      message = err;

    console.log('Build failed: ', message);
    process.exit(1);
  }
}

build();
