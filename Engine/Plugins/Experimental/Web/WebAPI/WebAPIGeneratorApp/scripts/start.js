const fs = require('fs');
const path = require('path');
const cp = require('child_process');

const root = path.resolve(__dirname, '..');

function execute(command, cwd, description, output) {
  return new Promise((resolve, reject) => {
    const child = cp.exec(command, { encoding: 'utf8', cwd });
    child.addListener('error', reject);
    child.addListener('exit', code => {
      if (code === 0)
        resolve();
      else
        reject(new Error(`Failed To ${description}`));
    });

    if (output)
      child.stdout.on('data', console.log);
  });
}

function printError(err) {
  let message = err.stdout;
  if (!message)
    message = err.message;
  if (!message)
    message = err;

  console.log('ERROR:', message);
}

async function build() {
  try {
    const build = path.join(root, 'build');
    const installed = path.join(build, 'version.txt');
    const pkgPath = path.join(root, '_package.json');

    if (!fs.existsSync(pkgPath)) {
      console.log("ERROR: Failed to build WebApp - Can't find package.json")
      return false;
    }

    const pkg = JSON.parse(fs.readFileSync(pkgPath, 'utf8'));
    if (!pkg || !pkg.version) {
      console.log("ERROR: Failed to build WebApp - Can't parse package.json")
      return false;
    }

	const forceBuild = !!process.argv.find(arg => arg === '--build');
	if (!forceBuild && fs.existsSync(installed)) {
      const version = fs.readFileSync(installed, 'utf8');
      if (version && pkg.version === version)
        return true;
    }

    for (const folder of [root]) {
      for (const filename of ['package.json', 'package-lock.json']) {
        const source = path.join(folder, `_${filename}`);
        const target = path.join(folder, filename);
        if (!fs.existsSync(source))
          continue;

        if (fs.existsSync(target))
          fs.unlinkSync(target);

        fs.copyFileSync(source, target);
        const stat = fs.statSync(target);

        // Remove readonly flag set by perforce
        fs.chmodSync(target, stat.mode | 0o600);
      }
    }

    console.log('Installing dependencies');
    await execute('npm install', root, 'Install dependencies');

    console.log('Building WebApp');
    await execute('npm run build', root, 'Build WebApp', true);
    if (!fs.existsSync(build))
        return false;

    fs.writeFileSync(installed, pkg.version, 'utf8');
    return true;

  } catch (err) {
    printError(err);
    return false;
  }
}

async function start() {
  try {
    console.log('Starting WebApp...');
    const args = process.argv.slice(2).map(arg => `"${arg}"`).join(' ');
    const compiled = path.join(root, 'build');
    await execute(`node "${compiled}" ${args}`, root, 'Run WebApp', true);
  } catch (err) {
    printError(err);
  }
}

async function buildAndStart() {
  if (await build())
    await start();
}

buildAndStart();
