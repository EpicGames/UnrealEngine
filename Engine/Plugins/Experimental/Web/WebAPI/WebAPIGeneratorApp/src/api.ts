// properties required by template, may be different per-template
interface TemplateContext {

}

// common properties for many contexts
interface TemplateContextShared {
    namespacePrefix: string // ie. SomeNamespace -> FSomeNamespace_ClassName
}

interface GenerateContext {
    templateName: string // file name without path or extension
    outputPath: string // can be relative or absolute

    templateContext: TemplateContext | null
}

interface GenerateContextCollection {
    outputPathCommon: string // All other paths, if relative, are relative to this
    items: GenerateContext[]

    templateContextShared: TemplateContextShared | null
}

// @note: should probably be websocket server - caller will want 
// progress and any warnings/errors as they arise, not just a final report


