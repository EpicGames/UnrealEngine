// Define ENDPOINT

const BASE_ENDPOINT = "/api";
const API_VERSION = `${BASE_ENDPOINT}/v1`;
const SERVER_STATUS_ENDPOINT = `${BASE_ENDPOINT}/server-status`;

module.exports = {
    BASE_ENDPOINT,
    API_VERSION,
    SERVER_STATUS_ENDPOINT,
}