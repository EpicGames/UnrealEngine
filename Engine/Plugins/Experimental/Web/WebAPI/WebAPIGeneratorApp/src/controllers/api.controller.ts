import { Controller, Get } from '@overnightjs/core';
import { Request, Response } from 'express';

interface ISendOptions {
  type?: string;
  nullIsNotFound?: boolean;
}

@Controller('api')
export class ApiController {
  public async initialize(): Promise<void> {
  }

  @Get('ping')
  private ping(req: Request, res: Response) {
    res.send({ message: 'ok' });
  }

  @Get('shutdown')
  private shutdown(req: Request, res: Response) {
    res.send({ message: 'ok' });
    setImmediate(() => process.exit(0));
  }

  @Get('status')
  private status(req: Request, res: Response) {
    res.send({ message: 'ok' });
  }

  protected async send(res: Response, promise: Promise<any>, options?: ISendOptions): Promise<any> {
    try {
      const result = await promise;
      if (result === null && options?.nullIsNotFound) {
        return res.status(404).send({ status: 'ERROR', error: 'Not found' });
      }

      if (options?.type) {
        res.set('Content-Type', options.type);
      }

      res.send(result);

    } catch (err: any) {
      const error = err?.message ?? err;
      res.status(500)
          .send({ status: 'ERROR', error });
    }
  }
}
