import { Controller, Post } from '@overnightjs/core';
import { Request, Response } from 'express';
const catchAsync_ = require('../utils/catchAsync');
const { generatorService } = require('../services');
const httpStatus = require('http-status');

interface ISendOptions {
    type?: string;
    nullIsNotFound?: boolean;
}

@Controller('api/gen')
export class GeneratorController {
    public async initialize(): Promise<void> {
    }

    @Post('struct')
    private async struct(req: Request, res: Response) {
        const generatedStruct = await generatorService.generateStruct(req.body);
        res.status(httpStatus.OK).send(generatedStruct);
    }

    @Post('enum')
    private async enum(req: Request, res: Response) {
        const generatedEnum = await generatorService.generateEnum(req.body);
        res.status(httpStatus.OK).send(generatedEnum);
    }

    @Post('header')
    private async header(req: Request, res: Response) {
        const generatedHeader = await generatorService.generateHeader(req.body);
        res.status(httpStatus.OK).send(generatedHeader);
    }

    @Post('call')
    private async call(req: Request, res: Response) {
        const generatedCall = await generatorService.generateCall(req.body);
        res.status(httpStatus.OK).send(generatedCall);
    }

    @Post('asyncAction')
    private async asyncAction(req: Request, res: Response) {
        const generatedAsyncAction = await generatorService.generateAsyncAction(req.body);
        res.status(httpStatus.OK).send(generatedAsyncAction);
    }

    @Post('decl')
    private async decl(req: Request, res: Response) {
        const generatedDecl = await generatorService.generateDecl(req.body);
        res.status(httpStatus.OK).send(generatedDecl);
    }

    @Post('defn')
    private async defn(req: Request, res: Response) {
        const generatedDefn = await generatorService.generateDefn(req.body);
        res.status(httpStatus.OK).send(generatedDefn);
    }

    protected async send(res: Response, promise: Promise<any>, options?: ISendOptions): Promise<any> {
        try {
            const result = await promise;
            if (result === null && options?.nullIsNotFound) {
                return res.status(404).send({ status: 'ERROR', error: 'Not found' });
            }

            if (options?.type) {
                res.set('Content-Type', options.type);
            }

            res.send(result);

        } catch (err: any) {
            const error = err?.message ?? err;
            res.status(500)
                .send({ status: 'ERROR', error });
        }
    }
}
