// The underscore is required because TypeScript is stupid
const catchAsync_ = require("../utils/catchAsync");
const { generatorService } = require("../services");
const httpStatus = require("http-status");

const requestStruct = catchAsync_(async (req: any, res: any) => {
    const generatedStruct = await generatorService.generateStruct(req.body);
    res.status(httpStatus.OK).send(generatedStruct);
});

const requestEnum = catchAsync_(async (req: any, res: any) => {
    const generatedEnum = await generatorService.generateEnum(req.body);
    res.status(httpStatus.OK).send(generatedEnum);
});

const requestHeader = catchAsync_(async (req: any, res: any) => {
    const generatedHeader = await generatorService.generateHeader(req.body);
    res.status(httpStatus.OK).send(generatedHeader);
});

const requestCall = catchAsync_(async (req: any, res: any) => {
    const generatedCall = await generatorService.generateCall(req.body);
    res.status(httpStatus.OK).send(generatedCall);
});

const requestSubsystem = catchAsync_(async (req: any, res: any) => {
    const generatedSubsystem = await generatorService.generateSubsystem({});
    res.status(httpStatus.OK).send(generatedSubsystem);
});

const requestAsyncAction = catchAsync_(async (req: any, res: any) => {
    const generatedAsyncAction = await generatorService.generateAsyncAction(req.body);
    res.status(httpStatus.OK).send(generatedAsyncAction);
});

const requestDecl = catchAsync_(async (req: any, res: any) => {
    const generatedDecl = await generatorService.generateDecl(req.body);
    res.status(httpStatus.OK).send(generatedDecl);
});

const requestDefn = catchAsync_(async (req: any, res: any) => {
    const generatedDefn = await generatorService.generateDefn(req.body);
    res.status(httpStatus.OK).send(generatedDefn);
});

module.exports = {
    requestStruct,
    requestEnum,
    requestHeader,
    requestCall,
    requestSubsystem,
    requestAsyncAction,
    requestDecl,
    requestDefn,
};
