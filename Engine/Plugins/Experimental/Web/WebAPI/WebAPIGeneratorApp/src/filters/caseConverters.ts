// Usage: {{ items | titlecase }}
import { MetadataInfo } from '../types/metadata';

export function titleCase(v: string): string {
    if(!v || v.length === 0) { return ""; }

    let prefix = "";
    if(v.startsWith("(")) {
        prefix = "(";
        v = v.substring(1);
    }

    let suffix = "";
    if(v.endsWith(")")) {
        suffix = ")";
        v = v.substring(0, v.length - 1);
    }

    const result = v.replace(/([A-Z])/g, " $1").trim();
    return prefix + result.charAt(0).toUpperCase() + result.slice(1) + suffix;
}
