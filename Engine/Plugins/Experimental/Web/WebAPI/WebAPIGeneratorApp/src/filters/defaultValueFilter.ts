import { MetadataInfo } from '../types/metadata';
import { PropertyInfo } from '../types/property';
import { NameVariant, TypeInfo, toString } from '../types/typeInfo';

/** Returns the default value for the given object */
// Usage: {{ item | defaultvalue }}
export function defaultValue(v: PropertyInfo): string {
    if(!v) {
        return "";
    }

    if(v.defaultValue)
    {
        return v.defaultValue
    }

    if(typeof v.type != 'string')
    {
        if(!v.type.isBuiltinType)
        {
            if(v.type.prefix == "F")
            {
                return `${toString(v.type)}::GetDefault()`;
            }
            else if(v.type.prefix == "U")
            {
                return "nullptr;"
            }
        }
    }

    return '{}'
}
