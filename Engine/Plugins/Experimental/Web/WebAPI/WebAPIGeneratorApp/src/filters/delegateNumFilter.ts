// Usage: {{ num | numStr }}
export function numStr(v: number): string {
    if(v == 0 || v > 8) {
        return "";
    }

    const strLookup: Array<string> = [
        "One", "Two", "Three", "Four", "Five", "Six", "Seven", "Eight"
    ];

    return strLookup[v - 1] + (v > 1 ? "Params" : "Param");
}
