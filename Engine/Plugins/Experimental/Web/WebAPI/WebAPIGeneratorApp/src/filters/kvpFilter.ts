// Usage: {{ items | kvp %}}
import { MetadataInfo } from '../types/metadata';

export function keyValuePairs(v: Array<MetadataInfo | string>, prefix?: string, suffix?: string): string {
    if(!v || v.length === 0) { return ""; }
    const keyValuePairs = v.map((v: MetadataInfo | string) => {
        if(typeof v === "string") {
            return `${v}`;
        }
        else {
            return "value" in v && v.value ? `${v.key}="${v.value}"` : `${v.key}`;
        }
    });
    prefix = prefix ?? "";
    suffix = suffix ?? "";
    return `${prefix}${keyValuePairs.join(", ")}${suffix}`;
}
