// Usage: {% parameters | params %}
import { OperationParameterInfo } from '../types/operation';
import { DelegateInfo } from '../types/delegate';

// mode = undefined, decl, defn, macro or call
export function params(v: Array<OperationParameterInfo | DelegateInfo>, mode?: string): string {
    mode = mode ?? 'decl';
    const keyValuePairs = v.map((v: OperationParameterInfo | DelegateInfo) => {
        if('type' in v) {
            let prefix = mode === 'decl' ? `UPARAM(DisplayName = "${v.name}") ` : "";
            let suffix = mode === 'decl' ? v.defaultValue ? ` = ${v.defaultValue}` : "" : "";
            return mode === 'call'
                ?  `${v.name}`
                : mode === 'macro'
                ? `${v.isConst ? "const " : ""}${v.type}${v.accessModifier ?? ""}, ${v.name}`
                : `${prefix}${v.isConst ? "const " : ""}${v.type}${v.accessModifier ?? ""} In${v.name}${suffix}`;
        }
        else {
            let prefix = mode === 'decl' ? `UPARAM(DisplayName = "On${v.name}") ` : "";
            return mode === 'call'
                ? `InOn${v.name}`
                : `${prefix}FOn${v.name}Delegate& InOn${v.name}`;
        }
    });
    return keyValuePairs.join(", ");
}
