// Sets the specified property (or adds it) of one or more objects, to the specified value

// Usage: {{ items | set: "propertyName", 55 }}
export function set(arr: {[key: string]: any}[], property: string, value: any): {[key: string]: any}[] {
    arr.forEach((obj: {[key: string]: any}) => {
        obj[property] = value;
    });
    return arr;
}
