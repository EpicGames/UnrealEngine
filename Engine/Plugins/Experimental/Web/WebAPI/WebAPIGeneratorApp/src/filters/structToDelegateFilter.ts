// Usage: {% parameters | params %}
import { OperationParameterInfo, OperationResponseInfo } from '../types/operation';
import { DelegateInfo, DelegateParameterInfo } from '../types/delegate';
import { StructInfo } from '../types/struct';
import { PropertyInfo } from '../types/property';

// mode = undefined, decl, defn, macro or call
export function structToDelegate(v: Array<StructInfo | OperationResponseInfo>): DelegateInfo[] {
    if(!v) {
        return [];
    }
    return v.map((obj: StructInfo | OperationResponseInfo) => {
        const parameters = obj.properties
            ? obj.properties.map((p: PropertyInfo) => {
                return p as DelegateParameterInfo;
            })
            : [];
        return <DelegateInfo>{
            name: obj.name,
            parameters: parameters
        };
    });
}
