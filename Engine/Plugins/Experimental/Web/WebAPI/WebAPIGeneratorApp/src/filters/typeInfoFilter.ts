import { MetadataInfo } from '../types/metadata';
import { NameVariant, TypeInfo, toString } from '../types/typeInfo';

// Usage: {{ item | kvp }}
export function toStr(v: NameVariant | TypeInfo, justName?: boolean): string {
    return toString(v, justName);
}

// Usage: {{ item | jsonname }}
export function jsonName(v: NameVariant | TypeInfo): string {
    if(!v) {
        return "";
    }
    else if(typeof v == 'string') {
        return v;
    }
    else {
        return v.jsonName ?? v.name;
    }
}

// Usage: {{ item | jsontype }}
export function jsonType(v: NameVariant | TypeInfo): string {
    if(!v) {
        return "";
    }

    let value = v;
    if(typeof v == 'string') {
        value = v;
    }
    else {
        value = v.jsonType ?? v.name;
    }

    const validValues = [ "null", "string", "number", "boolean", "bool", "array", "object" ];

    if(validValues.indexOf(value.toLowerCase()) < 0)
    {
        // Not a valid json type (might be "ToFromHandler") so return empty string
        value = "";
    }

    return value;
}
