import { isTruthy } from "liquidjs";

// Extension of the default "where" filter, that allows an operator other than "===", and a defaultValue to compare against if the property resolves to undefined
export function where<T extends object>(arr: T[], property: string, expected?: any, operator?: string, defaultValue?: any): T[] {
    if(!arr) {
        return [];
    }

    let filtered = arr.filter((obj: T) => {
        const objWithKeys: {[key: string]: any} = obj
        let propertyParts = property.split('.')
        let value = objWithKeys
        for (let index = 0; index < propertyParts.length; index++) {
            const propertyPart = propertyParts[index]
            value = value[propertyPart]
            if(value === undefined && defaultValue !== undefined) {
                value = defaultValue
            }
        }

        if(value === undefined) {
            return false;
        }

        if(expected !== undefined) {
            operator = operator || "==="
            const expression = (left: any, right: any) => {
                let str = `return function(left, right) {
                    "use strict";
                    return (left ${operator} right);
                }`
                let functor = Function(str)
                return functor()
            };
            const expressionResult = expression(value, expected)(value, expected)
            return expressionResult
        }
        else {
            return value ? true : false
        }
    })

    return filtered;
}
