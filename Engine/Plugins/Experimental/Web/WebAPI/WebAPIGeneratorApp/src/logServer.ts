import ws from 'ws';
import _ from 'lodash';
import fs from 'fs-extra';
import { ServiceSettings } from './settings';

export namespace LogServer {
    var server: ws.Server | null;
    var buffer: string = '';
    var startTime = new Date().toISOString().replace(/\:/g, '_');
    var flushTimer: NodeJS.Timeout | null;

    export async function initialize(): Promise<void> {
        const port = ServiceSettings.dev ? 7002 : ServiceSettings.port + 2;
        server = new ws.Server({ port });
        await fs.ensureDir('./logs');
        server.on('connection', (socket: ws) => {
            socket.on('message', onMessage);
        });

        flushTimer = setInterval(flusLog, 2 * 60 * 1000);
    }

    export async function shutdown(): Promise<void> {
        server?.removeAllListeners();
        server?.close();
        server = null;
        if(flushTimer) {
            clearInterval(flushTimer);
        }
        flushTimer = null;
    }

    function onMessage(data: ws.Data) {
        const json = data.toString();
        try {
            const payload = JSON.parse(json);
            log(payload);
        } catch (err) {
            console.log("Error with message:", json);
            console.log(err);
        }
    }

    export function log(payload: Record<string, any>): void {
        const Timestamp = new Date().toISOString();
        buffer += JSON.stringify({ Timestamp, ...payload }) + '\r\n';
    }

    function flusLog() {
        if (flushLogFile(buffer, 'all'))
            buffer = '';
    }

    function flushLogFile(content: string, type: string) {
        if (!content)
            return false;

        try {
            const filename = `./Logs/${startTime}-${type}.log`;
            fs.appendFileSync(filename, content, 'utf8');

            return true;
        } catch (err) {
            console.log('error', err);
        }

        return false;
    }
}
