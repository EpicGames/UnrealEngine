const express = require('express');
const generatorController = require('../../controllers/generator.controller');
const router = express.Router();

router.route(`/struct`).post(generatorController.requestStruct);
router.route('/enum').post(generatorController.requestEnum);
router.route('/header').post(generatorController.requestHeader);
router.route('/call').post(generatorController.requestCall);
router.route('/asyncAction').post(generatorController.requestAsyncAction);
router.route('/decl').post(generatorController.requestDecl);
router.route('/defn').post(generatorController.requestDefn);

module.exports = router;
