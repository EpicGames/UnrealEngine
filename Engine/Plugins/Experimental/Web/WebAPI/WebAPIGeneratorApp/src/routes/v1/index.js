const express = require('express');
const generatorRoutes = require('./generators.routes');

const router = express.Router();

const defaultRoutes = [
    {
        path: '/gen',
        route: generatorRoutes
    }
];

defaultRoutes.forEach(route => {
    router.use(route.path, route.route);
});

module.exports = router;