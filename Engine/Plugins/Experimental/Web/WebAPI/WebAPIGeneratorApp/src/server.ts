import { Server } from '@overnightjs/core';
import { Request, Response, NextFunction, static as expressStatic } from 'express';
import bodyParser from 'body-parser';
import path from 'path';
import request from 'superagent';
import { ServiceSettings } from './settings';
import { LogServer } from './logServer';
import { GeneratorController } from './controllers/generator.controller';
import { ApiController } from './controllers/api.controller';

export class App extends Server {
    constructor() {
        super();

        this.app.use(bodyParser.json());
        this.app.use(bodyParser.urlencoded({ extended: false }));
        this.app.use((req, res, next) => App.setAccessControl(res, next));
        this.app.use(expressStatic(path.join(ServiceSettings.rootFolder, 'public'), { setHeaders: (res) => App.setAccessControl(res) }));
    }

    public async start(): Promise<void> {
        try {
            const apiController = new ApiController();
            apiController.initialize();

            const generatorController = new GeneratorController();
            generatorController.initialize();

            super.addControllers([apiController, generatorController]);

            this.app.use('/api', (req, res) => App.notFoundHandler(req, res, true));

            // Trying to kill a zombie process
            await request.get(`http://127.0.0.1:${ServiceSettings.port}/api/shutdown`)
                .timeout(100)
                .then(() => new Promise(resolve => setTimeout(resolve, 100)))
                .catch(() => { });

            await this.startServer();
            console.log('DONE: WebApp started, port:', ServiceSettings.port);

        } catch (error: any) {
            console.error('ERROR:', error.message);
            throw error;
        }
    }

    private startServer(): Promise<void> {
        return new Promise((resolve, reject) => {
            const server = this.app.listen(ServiceSettings.port, async () => {
                await LogServer.initialize();
                resolve();
            });

            server.once('error', reject);
        });
    }

    private static setAccessControl(res: Response, next?: NextFunction): void {
        res.header('Access-Control-Allow-Origin', '*');
        res.header('Access-Control-Allow-Headers', '*');
        res.header('Access-Control-Max-Age', '86400');
        res.header('Access-Control-Allow-Methods', 'GET, POST, PUT, DELETE, OPTIONS');

        next?.();
    }

    private static notFoundHandler(req: Request, res: Response, isApi: boolean): void {
        res.status(404)
            .send({ error: `Invalid ${isApi ? 'api call' : 'request'}: ${req.method} '${req.originalUrl}'` });
    }
}

/*
import express, { Express } from 'express';
import morgan from 'morgan';
import helmet from 'helmet';
import cors from 'cors';
import config from '../config.json';
import { Liquid } from 'liquidjs';
import { request } from 'http';
import { ApiController } from './controllers/api.controller';
const routes = require('./routes/v1');
console.log(path.resolve(__dirname, 'templates/'))

const app: Express = express();
const engine: Liquid = new Liquid({
  root: path.resolve(__dirname, 'templates/'),
  extname: '.liquid',
  strictFilters: true
});

app.engine('liquid', engine.express())
app.set('json spaces', 4);
app.use(express.json());
app.use(express.urlencoded({ extended: true }));

await request.get(`https://127.0.0.1::${ServiceSettings.port}/api/${ServiceSettings.version}/shutdown`)

// Handle logs in console during development
if (process.env.NODE_ENV === 'development' || config.NODE_ENV === 'development') {
  app.use(morgan('dev'));
  app.use(cors());
}

// Handle security and origin in production
if (process.env.NODE_ENV === 'production' || config.NODE_ENV === 'production') {
  app.use(helmet());
}

app.use('/v1', routes);

app.use((err: Error, req: express.Request, res: express.Response, next: express.NextFunction) => {
  return res.status(500).json({
    errorName: err.name,
    message: err.message,
    stack: err.stack || 'no stack defined'
  });
});

export default app;
*/