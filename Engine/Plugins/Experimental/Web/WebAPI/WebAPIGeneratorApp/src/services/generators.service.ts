const getTemplateRoute = require("../utils/getTemplateRoute");
const generateMetadata = require("../utils/generateMetadata");

import { Liquid } from "liquidjs";
import { titleCase } from "../filters/caseConverters";
import { keyValuePairs } from "../filters/kvpFilter";
import { params } from "../filters/paramsFilter";
import { jsonName, toStr, jsonType } from "../filters/typeInfoFilter";
import { OperationInfo, OperationResponseInfo } from "../types/operation";
import { DeclInfo } from "../types/decl";
import { EnumInfo } from "../types/enum";
import { HeaderInfo } from "../types/header";
import { StructInfo } from "../types/struct";
import { where } from "../filters/whereFilter";
import { structToDelegate } from "../filters/structToDelegateFilter";
import { numStr } from "../filters/delegateNumFilter";
import { set } from "../filters/setFilter";
import { defaultValue } from "../filters/defaultValueFilter";

// @fixme: duplicated in server.ts
const engine: Liquid = new Liquid({
    root: getTemplateRoute,
    extname: ".liquid",
    // strictFilters: true,
    // strictVariables: true
});

engine.registerFilter('kvp', keyValuePairs);
engine.registerFilter('params', params)
engine.registerFilter('str', toStr)
engine.registerFilter('jsonname', jsonName)
engine.registerFilter('jsontype', jsonType)
engine.registerFilter('titlecase', titleCase)
engine.registerFilter('structToDelegate', structToDelegate)
engine.registerFilter('where', where)
engine.registerFilter('numStr', numStr)
engine.registerFilter('set', set)
engine.registerFilter('defaultvalue', defaultValue)

const generateStruct = async (body: StructInfo) => {
    return await engine.renderFile("structDecl", { 'structDecl': body });
};

const generateEnum = async (body: EnumInfo) => {
    return await engine.renderFile("enumDecl", body);
};

const generateHeader = async (body: HeaderInfo) => {
    return await engine.renderFile("header", body);
};

const generateCall = async (body: OperationInfo) => {
    return await engine.renderFile("callDecl", body);
};

const generateAsyncAction = async (body: OperationInfo) => {
    let decl = await engine.renderFile("asyncActionDecl", body);
    let defn = await engine.renderFile("asyncActionDefn", body);
    return defn;
    //return [decl, defn]
    // return await engine.renderFile("asyncActionDecl", body);
};

const generateDecl = async (body: DeclInfo) => {
    body.includePaths.sort();
    // @todo: remove after testing
    body.operations.forEach(operation => {
        operation.logName = operation.logName || "LogTemp";
    });
    return await engine.renderFile("header", body);
};

const generateDefn = async (body: DeclInfo) => {
    body.includePaths.sort();
    // @todo: remove after testing
    body.operations.forEach(operation => {
        operation.logName = operation.logName || "LogTemp";
    });
    return await engine.renderFile("cpp", body);
};

module.exports = {
    generateStruct,
    generateEnum,
    generateHeader,
    generateCall,
    generateAsyncAction,
    generateDecl,
    generateDefn,
};
