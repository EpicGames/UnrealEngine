import path from 'path';

class ServiceSettings {
  readonly name: string = 'WebAPILiquidJS';
  readonly version: string = 'v1';
  readonly dev: boolean;
  readonly monitor: boolean;
  readonly port: number;
  readonly ueHttpPort: number;
  readonly ueWebSocketPort: number;
  readonly rootFolder: string;
  readonly logger: boolean;

  constructor() {
    this.dev = false;
    this.monitor = false;
    this.port = 33000;
    this.ueHttpPort = 33010;
    this.ueWebSocketPort = 33020;
    this.logger = false;

    for (let i = 2; i < process.argv.length; i++) {
      switch (process.argv[i]) {
        case '--dev':
          this.dev = true;
          this.port = 33001;
          this.logger = true;
          break;

        case '--monitor':
          this.monitor = true;
          break;

        case '--port':
          this.port = parseInt(process.argv[i + 1]) || 33000;
          i++;
          break;

        case '--uehttp':
          this.ueHttpPort = parseInt(process.argv[i + 1]) || 33010;
          i++;
          break;

        case '--uews':
          this.ueWebSocketPort = parseInt(process.argv[i + 1]) || 33020;
          i++;
          break;

        case '--log':
          this.logger = true;
          break;
      }
    }

    process.env.NODE_ENV = this.dev ? 'development' : 'production';
    this.rootFolder = path.resolve('./');
  }
}

const Instance = new ServiceSettings();
export { Instance as ServiceSettings };