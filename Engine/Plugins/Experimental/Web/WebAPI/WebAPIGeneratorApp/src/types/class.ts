import { FunctionInfo } from "./function";
import { MetadataInfo } from "./metadata";
import { PropertyInfo } from "./property";
import { NameVariant } from "./typeInfo";

export interface ClassInfo {
    description: string;
    name: NameVariant;
    namespace: string;
    module: string;
    specifiers: Array<MetadataInfo> | string;
    metadata: Array<MetadataInfo> | string;
    properties: Array<PropertyInfo>;
    functions: Array<FunctionInfo>;
}
