import { ClassInfo } from "./class";
import { EnumInfo } from "./enum";
import { OperationInfo, OperationRequestInfo, OperationResponseInfo } from "./operation";
import { SettingsClassInfo } from "./settings";
import { StructInfo } from "./struct";

/** Wraps multiple types in a single header. */
export interface DeclInfo {
    baseFilePath?: string;
    relativeFilePath: string;
    fileName: string;
    fileType: string;
    namespace: string;
    copyrightNotice: string;
    module: string;
    moduleDependencies?: Array<string>;
    includePaths: Array<string>;
    forwardDeclarations?: Array<string>;
    enums: Array<EnumInfo>;
    structs: Array<StructInfo>;
    requests: Array<OperationRequestInfo>;
    responses: Array<OperationResponseInfo>;
    classes: Array<ClassInfo>;
    settings?: Array<SettingsClassInfo>;
    operations: Array<OperationInfo>
}
