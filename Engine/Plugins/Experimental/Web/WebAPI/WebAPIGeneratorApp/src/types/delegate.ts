import { PropertyInfo } from "./property";
import { NameVariant } from "./typeInfo";

export interface DelegateParameterInfo extends PropertyInfo {
    isConst?: boolean; // if unspecified, false
    accessModifier?: string // * or &
}

export interface DelegateInfo {
    name: NameVariant;
    parameters: Array<DelegateParameterInfo>;
    isDynamic?: boolean;
    isMulticast?: boolean;
}
