import { NameVariant } from "./typeInfo";

interface EnumValueInfo {
    name: NameVariant;
    jsonName?: NameVariant;
    displayName?: NameVariant;
    description?: string;
    explicitValue?: Number;
}

export interface EnumInfo {
    description: string;
    name: NameVariant;
    namespace: string;
    module: string;
    values: Array<EnumValueInfo | string>;
}
