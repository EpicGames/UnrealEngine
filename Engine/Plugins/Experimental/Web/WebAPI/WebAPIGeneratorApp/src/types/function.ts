import { MetadataInfo } from "./metadata";
import { PropertyInfo } from "./property";
import { NameVariant } from "./typeInfo";

interface FunctionParameterInfo extends PropertyInfo {
    isConst?: boolean; // if unspecified, false
    accessModifier?: string // * or &
}

export interface FunctionInfo {
    name: NameVariant;
    description: string;
    namespace: string;
    module: string;
    parameters: Array<FunctionParameterInfo>;
    isOverride?: boolean;
    isConst?: boolean;

    returnType: string;
    isReturnTypeConst?: boolean; // if unspecified, false
    body: string;

    specifiers: Array<MetadataInfo> | string;
    metadata: Array<MetadataInfo> | string;

    includes: Array<string>;
}
