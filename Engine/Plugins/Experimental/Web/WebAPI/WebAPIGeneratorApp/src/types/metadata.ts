// metadata is a key/value pair, or just a Key with no value
// ex: { key: "DisplayName", value: "MyFunction" }
export interface MetadataInfo {
    key: string
    value?: string
}
