import { DelegateInfo } from "./delegate";
import { StructInfo } from "./struct";
import { NameVariant } from "./typeInfo";

enum ParameterStorage {
    Path,
    Query,
    Header,
    Cookie,
    Body
}

export interface OperationParameterInfo {
    name: string;
    type: string;
    description: string;
    isConst?: boolean; // if unspecified, false
    accessModifier?: string // * or &
    defaultValue?: any;
    storage: ParameterStorage;
    mediaType?: string;
}

export interface OperationRequestInfo extends StructInfo {
    body?: StructInfo;
}

export interface OperationResponseInfo extends StructInfo {
    responseCode: number;
    isArray: boolean;
    message?: string;
}

export interface OperationInfo {
    description: string;
    namespace: string;
    service: string;
    path: string;
    module: string;
    name: NameVariant;
    verb: string;
    returnType: string;
    errorType: string;
    delegates?: Array<DelegateInfo>;
    parameters: Array<OperationParameterInfo>;
    requests: Array<OperationRequestInfo>
    responses: Array<OperationResponseInfo>

    includes: Array<string>;
    filenameWithoutExtension: string;
    settingsTypeName?: string;

    logName?: string;
}
