import { MetadataInfo } from "./metadata";
import { NameVariant } from "./typeInfo";

export interface PropertyInfo {
    name: NameVariant;
    description: string;
    namespace: string;
    type: NameVariant;
    isArray: boolean;
    isRequired: boolean;
    isMixin?: boolean;
    defaultValue?: string;
    specifiers: Array<MetadataInfo> | string;
    metadata: Array<MetadataInfo> | string;
};
