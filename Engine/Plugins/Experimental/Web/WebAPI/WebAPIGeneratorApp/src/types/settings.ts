import { NameVariant } from "./typeInfo";

export interface SettingsClassInfo {
    name: NameVariant;
    base?: NameVariant;
    displayName?: string;
    host: string;
    baseUrl?: string;
    userAgent: string;
    dateTimeFormat?: string;
    schemes: Array<string>;
}

// const a = { hat: "h"};
// const b = { hat: "w"};
// const obj: SettingsClassInfo = {...a, ...b};
