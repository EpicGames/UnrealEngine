import { MetadataInfo } from "./metadata";
import { PropertyInfo } from "./property";
import { NameVariant } from "./typeInfo";

export interface StructInfo {
    description: string;
    name: NameVariant;
    namespace: string;
    module: string;
    base?: NameVariant;
    specifiers: Array<MetadataInfo> | string;
    metadata: Array<MetadataInfo> | string;
    properties: Array<PropertyInfo>;
}
