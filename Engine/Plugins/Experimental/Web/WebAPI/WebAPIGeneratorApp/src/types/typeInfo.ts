export interface TypeInfo {
    name: string;
    jsonName: string;
    jsonType: string;
    jsonPropertyToSerialize?: string;
    printFormatSpecifier?: string;
    printFormatExpression?: string;
    namespace: string;
    prefix: string;
    containerType: string;
    isBuiltinType: boolean;

    toString(): string;
}

export type NameVariant = string | TypeInfo;

export function toString<T>(obj: T, justName?: boolean): string;

export function toString(info: NameVariant, justName?: boolean): string {
    if(!info) {
        return "";
    }
    else if(typeof info == 'string') {
        return info;
    }
    else {
        if(justName) {
            return info.name;
        }

        let containerType = info.containerType?.toLowerCase();
        if(!info.name && containerType.includes('enum')) {
            return "EUnknownEnum";
        }

        let typeStr = !info.namespace || info.isBuiltinType
            ? `${info.prefix}${info.name}`
            : `${info.prefix}${info.namespace}_${info.name}`;

        if(containerType) {
            let outerType = "";
            if(containerType.includes('array')) {
                outerType = "TArray";
            }
            else if(containerType.includes('set')) {
                outerType = "TSet";
            }
            else if(containerType.includes('map')) {
                outerType = "TMap";
            }

            if(outerType) {
                typeStr = `${outerType}<${typeStr}>`;
            }
        }

        return typeStr;
    }
}
