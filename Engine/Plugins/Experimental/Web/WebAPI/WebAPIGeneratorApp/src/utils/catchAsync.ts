const catchAsync = (callback: any) => (req: Request, res: Response, next: any) => {
    Promise.resolve(callback(req, res, next)).catch((error: any) => next(error));
};

module.exports = catchAsync;