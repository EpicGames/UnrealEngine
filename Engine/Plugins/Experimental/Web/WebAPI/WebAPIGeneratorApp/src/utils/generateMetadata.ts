import { MetadataInfo } from '../types/metadata';

const generateMetadata = (metadataMap: Array<MetadataInfo>): string => {
    let metadata_: string = "";
    metadataMap?.forEach((metadata: MetadataInfo) => {
        if (metadata_?.length !== 0) {
            metadata_ = `${metadata_}, ${metadata.key} = "${metadata.value}"`;
        } else {
            metadata_ = `meta = (${metadata.key} = "${metadata.value}"`;
        }
    })

    return `${metadata_})`;
}

module.exports = generateMetadata;