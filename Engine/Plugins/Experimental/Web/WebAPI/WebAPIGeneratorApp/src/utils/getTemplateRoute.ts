import path from 'path';

const getTemplateRoute = () => {
    let root = __dirname;
    const srcIdx = root.indexOf('src');    
    return path.resolve(`${root.substring(0, srcIdx)}`, 'templates/');
}

module.exports = getTemplateRoute();
