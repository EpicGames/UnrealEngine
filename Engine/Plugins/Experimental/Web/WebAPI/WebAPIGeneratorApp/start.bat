@echo off

rem Add default nodejs installation folder to path, in case it was not added / overwritten
SET PATH=%PATH%;%ProgramFiles%\nodejs\

rem Check if nodejs is in the env variable PATH
for %%X in (node.exe) do (set node=%%~$PATH:X)
if not defined node (
  echo ERROR: Couldn't find node.js installed, Please install latest nodejs from https://nodejs.org/en/download/
  exit 1
)

rem Let's check if it is a modern nodejs
node -e "process.exit( process.versions.node.split('.')[0] );"
echo Found Node.js version %errorlevel% (%node%)

if %errorlevel% LSS 14 (
  echo ERROR: installed node.js version is too old, please install latest nodejs from https://nodejs.org/en/download/
  exit 1
)

rem redirecting all command line arguments to node script
node scripts/start.js %*