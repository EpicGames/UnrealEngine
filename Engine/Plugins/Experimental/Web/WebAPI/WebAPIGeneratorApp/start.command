#!/bin/sh
# Copyright Epic Games, Inc. All Rights Reserved.
#
# Simple wrapper around start.sh using the
# .command extension enables it to be run from the OSX Finder.

sh "`dirname "$0"`"/start.sh $*
