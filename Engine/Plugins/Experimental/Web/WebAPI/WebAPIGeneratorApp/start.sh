#!/bin/sh

set -e

# First we check if nodejs is installed
if ! command -v node > /dev/null ; then
  echo "ERROR: Couldn't find node.js installed..., Please install latest nodejs from https://nodejs.org/en/download/"
  exit 1
fi

# Check if npm is installed
if ! command -v npm > /dev/null ; then
  echo "ERROR: Couldn't find npm installed..., Please install npm"
  exit 1
fi

# Let's check if it is a modern nodejs
VERSION=$(node -e "console.log( process.versions.node.split('.')[0] );")
echo "Found Node.js version ${VERSION}"

if [ ${VERSION} -lt 14 ] ; then
  echo "ERROR: installed node.js version is too old (${VERSION}) :\( Please install latest nodejs from https://nodejs.org/en/download/"
  exit 1
fi

FOLDER=$(dirname "$0")
node ${FOLDER}/scripts/start.js "$@"

