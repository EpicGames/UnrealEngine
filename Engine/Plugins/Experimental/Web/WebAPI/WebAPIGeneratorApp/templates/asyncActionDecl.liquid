/** {{ description }} */
UCLASS(
{{- property.specifiers | kvp: "", ", " -}}
{{- property.metadata | kvp: "meta=(", ")" -}})
class {{ moduleName | upcase | append: "_API" }} U{{ namespace }}_{{ name }} final
    : public UWebAPIOperationObject
{
	GENERATED_BODY()

public:
{%- for delegate in delegates %}
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FMOnResponse, FPetStore_FindPetsByStatus_Response_200, Response, FPetStore_FindPetsByStatus_Response_400, Error);
	DECLARE_DYNAMIC_DELEGATE_TwoParams(FOnResponse, FPetStore_FindPetsByStatus_Response_200, Response, FPetStore_FindPetsByStatus_Response_400, Error);


    DECLARE_DYNAMIC_MULTICAST_DELEGATE{{ delegate.parameters.length | delegateSuffix }}(FOn{{ delegate.name }}Delegate, {{ delegate.parameters | params: 'macro' }});

    UPROPERTY(BlueprintAssignable)
    F{{ delegate.name }}Delegate {{ delegate.name }};

{%- endfor %}

    // @todo: use generated rqeuest structs
	UFUNCTION(BlueprintCallable, meta = (BlueprintInternalUseOnly = "true"))
    static U{{ name }}* {{ name }}Async({{ parameters | params: "decl" }});

    UFUNCTION(BlueprintCallable, meta = (BlueprintInternalUseOnly = "true"))
    static void {{ name }}({{ parameters | params: "decl" }}, {{ delegates | params: "decl"}});

private:
    // @todo: use generated response structs
    {% assign responseType = TValueOrError<TTuple<uint32, TVariant<FResponse1, FResponse2>>, FError>> -%}
    static TFuture<{{ responseType }}> Request({{ parameters | params: "decl" }});

    static FString RequestToJson({{ parameters | params: "decl" }});
    static TTuple<uint32, TVariant<FResponse1, FResponse2>> ResponseFromJson(uint32 InResponseCode, const FString& InJsonString);

    FString GetBaseUrl();
};

/** {{ description }} */
UCLASS(
{{- property.specifiers | kvp: "", ", " -}}
{{- property.metadata | kvp: "meta=(", ")" -}})
UCLASS(BlueprintType, meta = (Namespace = "PetStore", Service = "Pet"))
class PETSTOREAPIGENERATED_API UPetStore_FindPetsByStatus
	: public UWebAPIOperationObject
{
	GENERATED_BODY()

public:
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FMOnResponse, FPetStore_FindPetsByStatus_Response_200, Response, FPetStore_FindPetsByStatus_Response_400, Error);
	DECLARE_DYNAMIC_DELEGATE_TwoParams(FOnResponse, FPetStore_FindPetsByStatus_Response_200, Response, FPetStore_FindPetsByStatus_Response_400, Error);

	UPROPERTY(BlueprintAssignable, meta = (Response = 200))
	FMOnResponse OnSuccess;

	UPROPERTY(BlueprintAssignable, meta = (Response = 400))
	FMOnResponse OnError;

	/** Finds Pets by status. */
	UFUNCTION(BlueprintCallable, meta = (BlueprintInternalUseOnly = true, DisplayName = "FindPetsByStatus"))
	static UPetStore_FindPetsByStatus* FindPetsByStatusAsync(UPARAM(DisplayName = "Request") const FPetStore_FindPetsByStatus_Request& InRequest);

	/** Finds Pets by status. */
	UFUNCTION(BlueprintCallable, meta = (BlueprintInternalUseOnly = true))
	static void FindPetsByStatus(UPARAM(DisplayName = "Request") const FPetStore_FindPetsByStatus_Request& InRequest, UPARAM(DisplayName = "OnSuccess") const FOnResponse& InOnSuccess, UPARAM(DisplayName = "OnError") const FOnResponse& InOnError);

private:
	using FResponseType = TPair<uint32, TVariant<FPetStore_FindPetsByStatus_Response_200, FPetStore_FindPetsByStatus_Response_400>>;
	static TFuture<UPetStore_FindPetsByStatus::FResponseType> Request(const FString& InBaseUrl, const FPetStore_FindPetsByStatus_Request& InRequest);

	static void FindPetsByStatusInternal(
		UPetStore_FindPetsByStatus* InOperation,
		const FPetStore_FindPetsByStatus_Request& InRequest,
		const FOnResponse& InOnSuccess = {},
		const FOnResponse& InOnError = {});

	FString GetBaseUrl() const;
};
