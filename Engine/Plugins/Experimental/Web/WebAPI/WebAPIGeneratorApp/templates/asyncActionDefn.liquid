#include "BlueprintAsyncHttpRequest.h"

#define LOCTEXT_NAMESPACE "{{ name }}"

{% capture className %}U{{ namespace }}_{{ name }}{% endcapture -%}
{% assign responseType = TValueOrError<TTuple<uint32, TVariant<FResponse1, FResponse2>>, FError>> -%}
UBlueprintAsyncHttpRequest* {{ className }}::{{ name }}Async({{ parameters | params: "defn" }})
{
	{{ className }}* Action = NewObject<{{ className }}>();
    AddToRoot();
    Request({{ parameters | params: "call" }})
    .Next([&, Action](const responseType& InResponse) {
        if(InResponse.HasError())
        {
            Action->OnError.Broadcast(InResponse.GetError());
        }
        else
        {
            Action->OnSuccess.Broadcast(InResponse.GetValue());
        }
	    RemoveFromRoot();
    });
	return Action;
}

{% capture className %}U{{ namespace }}_{{ name }}{% endcapture -%}
void {{ className }}::{{ name }}({{ parameters | params: "defn" }}, {{ delegates | params: "defn"}})
{
    Request({{ parameters | params: "call" }})
    .Next([&](const responseType& InResponse) {
        if(InResponse.HasError())
        {
            InOnError.Broadcast(InResponse.GetError());
        }
        else
        {
            InOnSuccess.Broadcast(InResponse.GetValue());
        }
    });
}

TFuture<{{ responseType }}> {{ className }}::Request({{ parameters | params: "defn" }})
{
    FSring RequestJsonStr = RequestToJson({{ parameters | params: "call" }});

	const auto Promise = MakeShared<TPromise<{{ responseType }}>>, ESPMode::ThreadSafe>();

	FHttpModule& HttpModule = FHttpModule::Get();

	TSharedRef<IHttpRequest, ESPMode::ThreadSafe> Request = HttpModule.CreateRequest();
	Request->SetURL(GetBaseUrl() + TEXT("{{ relativeUrl }}"));
	Request->SetVerb(TEXT("{{ verb | upcase }}")");
    Request->SetContentAsString(RequestJsonStr);
	Request->SetHeader(TEXT("User-Agent"), TEXT("X-UnrealEngine-Agent"));
	Request->SetHeader(TEXT("Content-Type"), TEXT("application/json"));
	Request->SetHeader(TEXT("Accepts"), TEXT("application/json"));
	Request->SetHeader(TEXT("Accept-Encoding"), TEXT("identity"));

	Request->OnProcessRequestComplete().BindLambda(
		[Promise](FHttpRequestPtr InRequest, FHttpResponsePtr InResponse, bool bInWasSuccessful)
		{
            if (bInWasSuccessful && InResponse.IsValid())
            {

                Promise->SetValue(MakeValue());
            }
            else
            {
                Promise->SetValue(MakeError());
            }
		});

	Request->ProcessRequest();

	return Promise->GetFuture();
}

FString {{ className }}::RequestToJson({{ parameters | params: "defn" }})
{
    // @todo: jsonify input parameters and return string
    return "";
}

TTuple<uint32, TVariant<FResponse1, FResponse2>> {{ className }}::ResponseFromJson(uint32 InResponseCode, const FString& InJsonString)
{
    TSharedPtr<FJsonObject> JsonObject;
    const TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(InJsonString);
    if(FJsonSerializer::Deserialize(Reader, JsonObject))
    {
        // @todo: deserialize based on InResponseCode
    }
    else
    {
        // @todo: log deserialization error
        return MakeTuple(MoveTemp(InResponseCode), {});
    }
}

FString {{ className }}::GetBaseUrl()
{
    // @todo: get base url from settings
    return TEXT("localhost");
}

#undef LOCTEXT_NAMESPACE
