<!-- Represents a single web call function -->

#pragma once

#include "CoreMinimal.h"
{%- for incl in includes %}
    #include "{{ incl }}.h"
{%- endfor%}

#include "{{ filenameWithoutExtension }}.generated.h"

TFuture<TValueOrError< {{ returnType  }}, F{{ namespace }}_{{ errorType }}>> {{ verb }}{{ name }}(
{%- for parameter in parameters %}
    UPARAM(DisplayName="{{ parameter.name }}", Description=" {{ parameter.description }}") const {{ parameter.type }}& In{{ parameter.name }},
{%- endfor%}
{
}
