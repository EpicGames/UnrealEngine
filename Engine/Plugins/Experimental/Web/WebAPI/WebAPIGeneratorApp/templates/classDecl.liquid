/** {{ classDecl.description }} */
UCLASS(
{{- classDecl.specifiers | kvp: "", ", " -}}
{{- classDecl.metadata | kvp: "meta=(", ")" -}})
class {{ classDecl.module | upcase }}_API {{ classDecl.name | str }} : public UObject
{
    GENERATED_BODY()

public:
{% render "subTemplates/functionDecl" for classDecl.functions as functionDecl %}
{% render "subTemplates/propertyDecl" for classDecl.properties as propertyDecl %}

{%- block public -%}{%- endblock %}

private:
{% block private -%}{%- endblock %}
};

