// {{ copyrightNotice }}

{% if relativeFilePath.length > 0 -%}
#include "{{ relativeFilePath | append: '/' }}{{ fileName }}.h"
{%- else -%}
#include "{{ fileName }}.h"
{%- endif %}

{%- for incl in includePaths %}
#include "{{ incl }}"
{%- endfor %}

#define LOCTEXT_NAMESPACE "{{ namespace }}_{{ fileName }}"

{% if enums.length > 0 %}
namespace UE
{
    namespace {{ namespace }}
    {
{% render "enumDefn" for enums as enumDefn -%}
    }
};
{% endif %}

{%- render "modelDefn" for structs as modelDefn %}

{%- render "modelClassDefn" for classes as modelDefn %}

{%- render "requestDefn" for requests as requestDefn %}

{%- render "responseDefn" for responses as responseDefn %}

{%- render "operationDefn" for operations as operationDefn %}

{%- if settings -%}
{%- render "settingsDefn" with settings as settingsDefn %}
{%- endif -%}

#undef LOCTEXT_NAMESPACE
