    /** {{ delegateDecl.description }} */
    DECLARE_
{%- if delegateDecl.isDynamic -%}
DYNAMIC_
{%- endif -%}
{%- if delegateDecl.isMulticast -%}
MULTICAST_
{%- endif -%}
DELEGATE
{{- delegateDecl.parameters.length | numStr | prefix: "_" -}}
(
    {{- delegateDecl.name | str }}
{%- for parameter in operationDecl.parameters -%}
    {%- if forloop.first -%}
    ,
    {%- endif -%}
    {{ parameter.type | str }},
    {%- if delegateDecl.isDynamic -%}
    {{ parameter.name | str }},
    {%- endif -%}
    {%- unless forloop.last -%}
    ,
    {%- else -%}
    {%- endunless -%}
{%- endfor -%}
)

