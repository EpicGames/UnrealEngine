/** {{ enumDecl.description }} */
UENUM(
{{- enumDecl.specifiers | kvp: "", ", " -}}
{{- enumDecl.metadata | kvp: "meta=(", ")" -}})
enum class {{ enumDecl.name | str }} : uint8
{
{%- render "subTemplates/enumValueDecl" for enumDecl.values as enumValueDecl %}
};

namespace UE
{
    namespace {{ enumDecl.name.namespace }}
    {
    	/** Converts the enum to string. */
    	FString ToString(const {{ enumDecl.name | str }}& InEnum);

    	/** Converts string to enum. */
    	void FromString(const FString& InString, {{ enumDecl.name | str }}& OutEnum);

        /** Converts the enum to Json. */
        TSharedPtr<FJsonValue> ToJson(const {{ enumDecl.name | str }}& InEnum);

        /** Converts Json to enum. */
        void FromJson(const TSharedPtr<FJsonValue>& InJson, {{ enumDecl.name | str }}& OutEnum);
    }
};

