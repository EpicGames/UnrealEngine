static TMap<{{ enumDefn.name | str }}, FString> {{ enumDefn.name | str: true }}ToJsonString =
{
{%- for value in enumDefn.values %}
    { {{ enumDefn.name | str }}::{{ value.name | str }}, TEXT("{{ value | jsonname }}") },
{%- endfor %}
};

static TMap<FString, {{ enumDefn.name | str }}> JsonStringTo{{ enumDefn.name | str: true }} =
{
{%- for value in enumDefn.values %}
    { TEXT("{{ value | jsonname }}"), {{ enumDefn.name | str }}::{{ value.name | str }} },
{%- endfor %}
};

FString ToString(const {{ enumDefn.name | str }}& InEnum)
{
	return {{ enumDefn.name | str: true }}ToJsonString[InEnum];
}

void FromString(const FString& InString, {{ enumDefn.name | str }}& OutEnum)
{
	OutEnum = JsonStringTo{{ enumDefn.name | str: true }}[InString];
}

TSharedPtr<FJsonValue> ToJson(const {{ enumDefn.name | str }}& InEnum)
{
    const TSharedPtr<FJsonValueString> JsonValue = MakeShared<FJsonValueString>({{ enumDefn.name | str: true }}ToJsonString[InEnum]);
    return JsonValue;
}

void FromJson(const TSharedPtr<FJsonValue>& InJson, {{ enumDefn.name | str }}& OutEnum)
{
    FString EnumValueString;
    if(InJson->TryGetString(EnumValueString))
    {
        if(const {{ enumDefn.name | str }}* FoundEnumValue = JsonStringTo{{ enumDefn.name | str: true }}.Find(EnumValueString))
        {
            OutEnum = *FoundEnumValue;
        }
    }
}
