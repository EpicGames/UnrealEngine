// {{ copyrightNotice }}

#pragma once

{%- for incl in includePaths %}
#include "{{ incl }}"
{%- endfor %}

#include "{{ fileName }}.generated.h"

{% for decl in forwardDeclarations %}
{{ decl }};
{%- endfor %}

{% render "enumDecl" for enums as enumDecl %}

{%- render "modelDecl" for structs as modelDecl %}

{%- render "modelClassDecl" for classes as modelDecl %}

{%- render "requestDecl" for requests as requestDecl %}

{%- render "responseDecl" for responses as responseDecl %}

{%- render "operationDecl" for operations as operationDecl %}

{%- if settings -%}
{%- render "settingsDecl" with settings as settingsDecl %}
{%- endif -%}
