{% layout "classDefn", classDecl: modelDefn %}

{%- block defns %}
{% render "subTemplates/structToJsonDefn", structDefn: modelDefn %}
{% render "subTemplates/structFromJsonDefn", structDefn: modelDefn %}
{% endblock %}