{% layout "structDecl", structDecl: modelDecl %}

{%- block public %}
{% render "subTemplates/structToJsonDecl", structDecl: modelDecl %}
{% render "subTemplates/structFromJsonDecl", structDecl: modelDecl %}
{% endblock %}

{%- block private %}
{% endblock %}