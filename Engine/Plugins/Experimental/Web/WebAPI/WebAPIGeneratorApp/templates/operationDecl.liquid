/** {{ operationDecl.description }} */
UCLASS(
{{- operationDecl.specifiers | kvp: "", ", " -}}
{{- operationDecl.metadata | kvp: "meta=(", ")" -}})
class {{ operationDecl.module | upcase | append: "_API" }} {{ operationDecl.name | str }} final
	: public UWebAPIOperationObject
{
	GENERATED_BODY()

public:
    {%- assign operationName = operationDecl.name | str: true -%}
    {%- assign request = operationDecl.requests.first -%}

    {%- assign responses = operationDecl.responses | where: "responseCode", 300, "<" -%}
    {%- assign response = responses.first -%}

    {%- assign errors = operationDecl.responses | where: "responseCode", 300, ">=" -%}
    {%- assign error = errors.first %}

	DECLARE_DYNAMIC_MULTICAST_DELEGATE_{{ operationDecl.responses.length | numStr }}(FMOnResponse,
{%- for response in operationDecl.responses -%}
{%- assign prefix = "Response" -%}
{%- if response.responseCode >= 300 -%}
{%- assign prefix = "Error" -%}
{%- endif -%}
    {{- response.name | str -}}, {{ prefix }}_{{- response.responseCode -}}
    {%- unless forloop.last -%}
    , 
    {%- else -%}
    {%- endunless -%}
{%- endfor -%});
	DECLARE_DYNAMIC_DELEGATE_{{ operationDecl.responses.length | numStr }}(FOnResponse,
{%- for response in operationDecl.responses -%}
{%- assign prefix = "Response" -%}
{%- if response.responseCode >= 300 -%}
{%- assign prefix = "Error" -%}
{%- endif -%}
    {{- response.name | str -}}, {{ prefix }}_{{- response.responseCode -}}
    {%- unless forloop.last -%}
    , 
    {%- else -%}
    {%- endunless -%}
{%- endfor -%});

{% assign successResponseCode = 200 -%}
{%- if response -%}
{%- assign successResponseCode = response.responseCode -%}
{%- endif %}
	UPROPERTY(BlueprintAssignable, meta = (ResponseCodes = "{{ responses | map: "responseCode" | concat: 200 | uniq | sort | join: ", "  }}"))
	FMOnResponse OnSuccess;

{% assign errorResponseCode = 404 -%}
{%- if error -%}
{%- assign errorResponseCode = error.responseCode -%}
{%- endif %}
	UPROPERTY(BlueprintAssignable, meta = (ResponseCodes = "{{ errors | map: "responseCode" | concat: 404 | uniq | sort | join: ", " }}"))
	FMOnResponse OnError;

	/** {{ operationDecl.description }} */
	UFUNCTION(BlueprintCallable, Category = "{{ operationDecl.namespace }}|{{ operationDecl.service }}", BlueprintInternalUseOnly, meta = (DisplayName = "{{ operationName }}"))
	static {{ operationDecl.name | str }}* {{ operationName }}Async(UPARAM(DisplayName = "Request") const {{ request.name | str }}& InRequest);

	/** {{ operationDecl.description }} */
	UFUNCTION(BlueprintCallable, Category = "{{ operationDecl.namespace }}|{{ operationDecl.service }}", BlueprintInternalUseOnly, meta = (DisplayName = "{{ operationName }}"))
	static {{ operationDecl.name | str }}* {{ operationName }}(UPARAM(DisplayName = "Request") const {{ request.name | str }}& InRequest, UPARAM(DisplayName = "Success") FOnResponse InOnSuccess, UPARAM(DisplayName = "Error") FOnResponse InOnError);

    static const FString Path;

private:
	using FResponseType = TTuple<uint32, bool, TVariant<
{%- for response in operationDecl.responses -%}
    {{- response.name | str -}}
    {%- unless forloop.last -%}
    , 
    {%- else -%}
    {%- endunless -%}
{%- endfor -%}>>;
	TFuture<{{ operationDecl.name | str }}::FResponseType> Request(const  {{ request.name | str }}& InRequest);

	void {{ operationName }}Internal(
		{{ operationDecl.name | str }}* InOperation,
		const {{ request.name | str }}& InRequest,
		const FOnResponse& InOnSuccess = {},
		const FOnResponse& InOnError = {});
};

