{%- assign operationTypeName = operationDefn.name | str %}
{%- assign operationName = operationDefn.name | str: true -%}
{%- assign request = operationDefn.requests.first -%}

{%- assign responses = operationDefn.responses | where: "responseCode", 300, "<" -%}
{%- assign response = responses.first -%}

{%- assign errors = operationDefn.responses | where: "responseCode", 300, ">=" -%}
{%- assign error = errors.first -%}

{%- assign successResponseCode = 200 -%}
{%- if response -%}
{%- assign successResponseCode = response.responseCode -%}
{%- endif -%}

{%- assign errorResponseCode = 404 -%}
{%- if error -%}
{%- assign errorResponseCode = error.responseCode -%}
{%- endif -%}

const FString {{ operationTypeName }}::Path = TEXT("{{ operationDefn.path }}");

{{ operationTypeName }}* {{ operationTypeName }}::{{ operationName }}Async(const {{ request.name | str }}& InRequest)
{
	UWebAPISubsystem* WebAPISubsystem = GEngine->GetEngineSubsystem<UWebAPISubsystem>(); 

	const TObjectPtr<{{ operationTypeName }}> Operation = WebAPISubsystem->MakeOperation<{{ operationTypeName }}>(GetDefault<{{ operationDefn.settingsTypeName }}>());
	Operation->{{ operationName }}Internal(Operation, InRequest);
	return Operation;
}

{{ operationTypeName }}* {{ operationTypeName }}::{{ operationName }}(
	const {{ request.name | str }}& InRequest,
	FOnResponse InOnSuccess,
	FOnResponse InOnError)
{
	UWebAPISubsystem* WebAPISubsystem = GEngine->GetEngineSubsystem<UWebAPISubsystem>();

	const TObjectPtr<{{ operationTypeName }}> Operation = WebAPISubsystem->MakeOperation<{{ operationTypeName }}>(GetDefault<{{ operationDefn.settingsTypeName }}>());
	Operation->{{ operationName }}Internal(Operation, InRequest, InOnSuccess, InOnError);
    return Operation;
}

void {{ operationTypeName }}::{{ operationName }}Internal(
	{{ operationTypeName }}* InOperation,
	const {{ request.name | str }}& InRequest,
	const FOnResponse& InOnSuccess,
	const FOnResponse& InOnError)
{
	InOperation->OnSuccess.Add(InOnSuccess);
	InOperation->OnError.Add(InOnError);

	Request(InRequest)
	.Next([&, InOperation](const {{ operationTypeName }}::FResponseType& InResponse)
	{
		const uint32 ResponseCode = InResponse.Get<0>();

        {% assign validResponses = operationDefn.responses | where: "responseCode", 0, '>' -%}
        {%- for response in validResponses -%}
        {%- unless forloop.first %}
        else if
        {%- else %}
        if
        {%- endunless -%}
        (ResponseCode == {{ response.responseCode }})
        {
            {% assign delegateType = "Error" -%}
            {%- if response.responseCode < 300 -%}
            {% assign delegateType = "Success" -%}
            {%- endif -%}
            InOperation->On{{ delegateType }}.Broadcast(
            {%- for possibleResponse in operationDefn.responses -%}
            {%- if possibleResponse.responseCode == response.responseCode -%}
            InResponse.Get<2>().Get<{{ possibleResponse.name | str }}>()
            {%- else -%}
            {}
            {%- endif -%}
            {%- unless forloop.last -%}
            , 
            {%- endunless -%}
            {%- endfor -%});
            InOperation->On{{ delegateType }}.Remove(InOn{{ delegateType }});
        }
        {%- endfor %}
        else
        {
            {% assign defaultResponse = operationDefn.responses | where: "responseCode", 0 -%}
            {% assign defaultResponse = defaultResponse[0] -%}
            // Handle all other responses
            if(ResponseCode > 0 && ResponseCode < 300)
            {
                InOperation->OnSuccess.Broadcast(
                {%- for possibleResponse in operationDefn.responses -%}
                {%- if possibleResponse.responseCode == defaultResponse.responseCode -%}
                InResponse.Get<2>().Get<{{ possibleResponse.name | str }}>()
                {%- else -%}
                {}
                {%- endif -%}
                {%- unless forloop.last -%}
                , 
                {%- endunless -%}
                {%- endfor -%});
                InOperation->OnSuccess.Remove(InOnSuccess);
            }
            else
            {
                InOperation->OnError.Broadcast(
                {%- for possibleResponse in operationDefn.responses -%}
                {%- if possibleResponse.responseCode == defaultResponse.responseCode -%}
                InResponse.Get<2>().Get<{{ possibleResponse.name | str }}>()
                {%- else -%}
                {}
                {%- endif -%}
                {%- unless forloop.last -%}
                , 
                {%- endunless -%}
                {%- endfor -%});
                InOperation->OnError.Remove(InOnError);
            }
        }

		UWebAPISubsystem* WebAPISubsystem = GEngine->GetEngineSubsystem<UWebAPISubsystem>();
		WebAPISubsystem->ReleaseOperation<{{ operationTypeName }}>(InOperation);
	});
}

TFuture<{{ operationTypeName }}::FResponseType> {{ operationTypeName }}::Request(const {{ request.name | str }}& InRequest)
{
	return RequestInternal(TEXT("{{ operationDefn.verb | upcase }}"), GetMutableDefault<{{ operationDefn.settingsTypeName }}>(), [&](const TSharedRef<IHttpRequest, ESPMode::ThreadSafe>& InHttpRequest)
	{
        const {{ operationDefn.settingsTypeName }}* Settings = GetDefault<{{ operationDefn.settingsTypeName }}>();
        FString RequestPath = Settings->FormatUrl({{ operationTypeName }}::Path);
{%- assign bodyProperties = request.properties | where: "storage", "Body" -%}
{%- if bodyProperties.length > 0 %}
        InHttpRequest->SetHeader(TEXT("Content-Type"), TEXT("application/json"));
		const TSharedPtr<FJsonValue> JsonValue = InRequest.ToJson();
		TSharedPtr<FJsonObject>* JsonObject = nullptr;
		TArray<TSharedPtr<FJsonValue>>* JsonValues;
		FString JsonString;
		if(JsonValue->TryGetObject(JsonObject))
		{
			const TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&JsonString);
			FJsonSerializer::Serialize((*JsonObject).ToSharedRef(), Writer);
		}
		else if(JsonValue->TryGetArray(JsonValues))
		{
			const TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&JsonString);
			FJsonSerializer::Serialize(*JsonValues, Writer);
		}
		InHttpRequest->SetContentAsString(JsonString);
{%- endif %}
{%- assign pathProperties = request.properties | where: "storage", "Path" -%}
{%- if pathProperties.length > 0 %}
        // Populate templated path
        RequestPath = InRequest.ToPath(RequestPath);
{%- endif %}
{%- assign queryProperties = request.properties | where: "storage", "Query" -%}
{%- if queryProperties.length > 0 %}
		FString QueryStr = InRequest.ToQuery();
		if(!QueryStr.IsEmpty())
		{
			RequestPath += TEXT("?") + QueryStr;
		}
{%- endif %}
		InHttpRequest->SetURL(RequestPath);

		if(Settings->bLogRequests)
		{
{% if headerProperties.length > 0 %}
			UE_LOG({{ operationDefn.logName }}, Display, TEXT("WebAPI Request: \nUrl: %s \nVerb: %s \nContent: %s"), *RequestPath, *InHttpRequest->GetVerb(), *JsonString);
{%- else %}
            UE_LOG({{ operationDefn.logName }}, Display, TEXT("WebAPI Request: \nUrl: %s \nVerb: %s"), *RequestPath, *InHttpRequest->GetVerb());
{%- endif %}
		}
	})
	.Next([&](const FRawResponse& InResponse)
	{
		using FResponseVariant = TTupleElement<2, {{ operationTypeName }}::FResponseType>::Type;
		FResponseVariant ResponseObject;

		uint32 ResponseCode = 0;

		if(!InResponse.Response.IsValid())
		{
			UE_LOG({{ operationDefn.logName }}, Display, TEXT("Http Request cancelled."));
			return MakeTuple(ResponseCode, false, MoveTemp(ResponseObject));
		}

		const FHttpResponsePtr& Response = InResponse.Response;
		ResponseCode = StaticCast<uint32>(Response->GetResponseCode());
        {% assign defaultResponse = operationDefn.responses | where: "responseCode", 0 -%}
        {% assign defaultResponse = defaultResponse[0] -%}
        {%- if defaultResponse -%}
        ResponseObject.Set<{{ defaultResponse.name | str }}>({{ defaultResponse.name | str }}());
        {%- endif -%}

		if(!InResponse.bWasSuccessful)
		{
			UE_LOG({{ operationDefn.logName }}, Error, TEXT("Http Request error."));
			return MakeTuple(ResponseCode, false, MoveTemp(ResponseObject));
		}
		else if(Response.IsValid())
		{
            {% assign validResponses = operationDefn.responses | where: "responseCode", 0, '>' -%}
            {%- for response in validResponses -%}
            {%- unless forloop.first -%}
            else if
            {%- else -%}
            if
            {%- endunless -%}
			(ResponseCode == {{ response.responseCode }})
			{
                {% if response.properties.length > 0 -%}
                {% if response.responseCode < 300 -%}
                {%- if response.isArray -%}
                TArray<TSharedPtr<FJsonValue>> JsonValues;
				if(FJsonSerializer::Deserialize(TJsonReaderFactory<TCHAR>::Create(Response->GetContentAsString()), JsonValues))
				{
                    TArray<{{ response.name | str }}> TypedResponses;
                    TypedResponses.Reserve(JsonValues.Num());
                    for(const TSharedPtr<FJsonValue>& JsonValue : JsonValues)
                    {
                        TypedResponses.Emplace_GetRef().FromJson(JsonValue);
                    }
					ResponseObject.Set<{{ response.name | str }}>(MoveTemp(TypedResponses));
				}
                {%- else -%}
                TSharedPtr<FJsonValue> JsonValue;
				if(FJsonSerializer::Deserialize(TJsonReaderFactory<TCHAR>::Create(Response->GetContentAsString()), JsonValue))
				{
                    {{ response.name | str }} TypedResponse;
                    TypedResponse.FromJson(JsonValue);
					ResponseObject.Set<{{ response.name | str }}>(MoveTemp(TypedResponse));
				}
                {%- endif %}
                {%- endif %}
                {%- else -%}
                ResponseObject.Set<{{ response.name | str }}>({{ response.name | str }}());
                {%- endif %}
			}
            {% endfor -%}
			else
			{
                UE_LOG({{ operationDefn.logName }}, Display, TEXT("Http ResponseCode %d was unexpected but handled by \"%s\" while requesting \"%s\": \"%s\""), ResponseCode, *{{ operationTypeName }}::StaticClass()->GetName(), *InResponse.Request->GetURL(), *InResponse.Response->GetContentAsString());
			}
		}

		return MakeTuple(ResponseCode, InResponse.bWasSuccessful, MoveTemp(ResponseObject));
	});
}
