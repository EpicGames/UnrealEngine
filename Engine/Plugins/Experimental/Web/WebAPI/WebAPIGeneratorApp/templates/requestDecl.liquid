{% layout "structDecl", structDecl: requestDecl %}

{%- block public %}
{%- assign bodyProperties = requestDecl.properties | where: "storage", "Body" -%}
{%- if bodyProperties.length > 0 %}
{% render "subTemplates/structToJsonDecl", structDecl: requestDecl %}
{% endif -%}
{%- assign pathProperties = requestDecl.properties | where: "storage", "Path" -%}
{%- if pathProperties.length > 0 %}
{% render "subTemplates/structToPathDecl", structDecl: requestDecl %}
{% endif -%}
{%- assign queryProperties = requestDecl.properties | where: "storage", "Query" -%}
{%- if queryProperties.length > 0 %}
{% render "subTemplates/structToQueryDecl", structDecl: requestDecl %}
{% endif -%}
{% endblock %}

{%- block private %}
{% endblock %}