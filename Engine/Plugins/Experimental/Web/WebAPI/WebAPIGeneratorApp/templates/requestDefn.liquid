{% layout "structDefn", structDefn: requestDefn %}

{%- block defns %}
{%- assign bodyProperties = requestDefn.properties | where: "storage", "Body" -%}
{%- if bodyProperties.length > 0 %}
{% render "subTemplates/structToJsonDefn", structDefn: requestDefn %}
{% endif -%}
{%- assign pathProperties = requestDefn.properties | where: "storage", "Path" -%}
{%- if pathProperties.length > 0 %}
{% render "subTemplates/structToPathDefn", structDefn: requestDefn %}
{% endif -%}
{%- assign queryProperties = requestDefn.properties | where: "storage", "Query" -%}
{%- if queryProperties.length > 0 %}
{% render "subTemplates/structToQueryDefn", structDefn: requestDefn %}
{% endif -%}
{% endblock %}