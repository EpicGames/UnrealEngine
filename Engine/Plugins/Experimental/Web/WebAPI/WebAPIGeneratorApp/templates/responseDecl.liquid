{% layout "structDecl", structDecl: responseDecl %}

{%- block public %}
{%- if responseDecl.properties.length > 0 %}
{% render "subTemplates/structFromJsonDecl", structDecl: responseDecl %}
{%- endif -%}
    static uint32 GetResponseCode();
    virtual const FText& GetMessage() const override;
{%- endblock -%}

{%- block private %}
    static const uint32 ResponseCode;
    static const FText Message;
{%- endblock -%}