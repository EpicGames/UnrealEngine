{% layout "structDefn", structDefn: responseDefn %}

{%- block defns -%} 
{%- if responseDefn.properties.length > 0 %}
{% render "subTemplates/structFromJsonDefn", structDefn: responseDefn %}
{%- endif -%}
constexpr uint32 {{ responseDefn.name | str }}::ResponseCode = {{ responseDefn.responseCode }};
const FText {{ responseDefn.name | str }}::Message = LOCTEXT("{{ responseDefn.name.namespace }}_{{ responseDefn.name.name }}_Message", "{{ responseDefn.message }}");

uint32 {{ responseDefn.name | str }}::GetResponseCode()
{
    return {{ responseDefn.name | str }}::ResponseCode;
}

const FText& {{ responseDefn.name | str }}::GetMessage() const
{
    return {{ responseDefn.name | str }}::Message;
}

{% endblock -%}