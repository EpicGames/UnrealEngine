/** {{ settingsDecl.description }} */
UCLASS(
{{- settingsDecl.specifiers | kvp: "", ", " -}}
{{- settingsDecl.metadata | kvp: "meta=(", ")" -}})
class {{ settingsDecl.module | upcase }}_API {{ settingsDecl.name | str }} : public U{{ settingsDecl.base | str }}
{
	GENERATED_BODY()

public:
    {{ settingsDecl.name | str }}();

	virtual FName GetContainerName() const override { return FName("Project"); }
	virtual FName GetCategoryName() const override { return FName("Plugins"); }
};
