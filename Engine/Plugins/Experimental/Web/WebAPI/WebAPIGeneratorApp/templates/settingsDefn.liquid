
{{ settingsDefn.name | str }}::{{ settingsDefn.name | str }}()
{
    Host = TEXT("{{ settingsDefn.host }}");
    BaseUrl = TEXT("{{ settingsDefn.baseUrl }}");
    UserAgent = TEXT("{{ settingsDefn.userAgent }}");
    DateTimeFormat = TEXT("{{ settingsDefn.dateTimeFormat }}");
    URISchemes = { {% for scheme in settingsDefn.schemes -%}TEXT("{{ scheme }}")
    {%- unless forloop.last -%}
    , 
    {%- else -%}
    {%- endunless -%}
    {%- endfor %} };
}
