/** {{ structDecl.description }} */
USTRUCT(
{{- structDecl.specifiers | kvp: "", ", " -}}
{{- structDecl.metadata | kvp: "meta=(", ")" -}})
struct {{ structDecl.module | upcase }}_API {{ structDecl.name | str }}
{%- if structDecl.base %} : public {{ structDecl.base | str -}}
{%- endif %}
{
    GENERATED_BODY()

{% render "subTemplates/propertyDecl" for structDecl.properties as propertyDecl %}

{%- block public -%}{%- endblock %}

{%- if structDecl.properties.length > 0 -%}
    static const {{ structDecl.name | str }}& GetDefault();

    // Equality Operators
	friend bool operator==(const {{ structDecl.name | str }}& A, const {{ structDecl.name | str }}& B)
	{
        return {% for property in structDecl.properties -%}
{%- capture propertyType -%}
{{ property.type | str }}
{%- endcapture -%}
{%- if propertyType == "FJsonObjectWrapper" and property.isArray == false -%}
            A.{{ property.name | str }}.JsonObject == B.{{ property.name | str }}.JsonObject
{%- else -%}
            A.{{ property.name | str }} == B.{{ property.name | str }}
{%- endif -%}
            {%- unless forloop.last %} && {% else -%}
            {%- endunless -%}
        {%- endfor -%}
        ;
	}

	friend bool operator!=(const {{ structDecl.name | str }}& A, const {{ structDecl.name | str }}& B)
	{
		return !(A == B);
	}
{%- endif -%}

private:
{% block private -%}{%- endblock %}
};

