{% block defns -%}{%- endblock %}

{%- if structDefn.properties.length > 0 -%}
const {{ structDefn.name | str }}& {{ structDefn.name | str }}::GetDefault()
{
    static {{ structDefn.name | str }} Default;
    return Default;
}
{%- endif %}
