    {{ enumValueDecl.name | str }}
    {%- if enumValueDecl.explicitValue -%}
    {{- enumValueDecl.explicitValue | prepend: " = " -}}
    {%- endif %}    UMETA(DisplayName="{{ enumValueDecl.displayName | str | titlecase }}", Description="{{ enumValueDecl.description }}", JsonName="{{ enumValueDecl.jsonName }}"),

