    /** {{ functionDecl.description }} */
    {% comment %}
    UFUNCTION(
    {{- functionDecl.specifiers | kvp: "", ", " -}}
    {{- functionDecl.metadata | kvp: "meta=(", ")" -}})
    {% endcomment %}
    {%- if functionDecl.isOverride -%} virtual {% endif -%}
    {%- if functionDecl.isReturnTypeConst %} const {% endif -%}
    {{ functionDecl.returnType | str }} {{ functionDecl.name | str -}}
    (
    {%- for parameter in functionDecl.parameters -%}
        {%- if forloop.first -%}
        ,
        {%- endif -%}
        {{ parameter.type | str }},
        {%- if delegateDecl.isDynamic -%}
        {{ parameter.name | str }},
        {%- endif -%}
        {%- unless forloop.last -%}
        ,
        {%- else -%}
        {%- endunless -%}
    {%- endfor -%}
    )
    {%- if functionDecl.isConst %} const {% endif -%}
    {%- if functionDecl.isOverride %} override {% endif -%}
    ;

