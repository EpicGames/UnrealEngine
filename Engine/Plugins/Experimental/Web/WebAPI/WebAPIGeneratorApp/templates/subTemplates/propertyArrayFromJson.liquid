        {%- assign isUPtr = property.type.prefix == 'U' -%}
        {%- assign propertyPath = property.name | str %}
        {%- if property.type.jsonPropertyToSerialize != '' %}
        {%- if isUPtr %}
        {%- assign propertyPath = property.name | str | append: "->" | append: property.type.jsonPropertyToSerialize %}
        {%- else %}
        {%- assign propertyPath = property.name | str | append: '.' | append: property.type.jsonPropertyToSerialize %}
        {%- endif -%}
        {%- endif %}
        {%- assign propertySignature = property.type | str -%}
        {%- if isUPtr -%}
        {%- assign propertySignature = propertySignature | prepend: "TObjectPtr<" | append: ">" -%}
        {%- endif %}
        {%- assign arrayPropertyPath = propertyPath %}
        {{ arrayPropertyPath }}.Reset({{ jsonArray }}->Num());
        for(const TSharedPtr<FJsonValue>& {{ property.name | str }}JsonItem : *{{ jsonArray }})
        {
            {%- assign jsonObject = property.name | str | append: 'JsonItem' %}
            {%- if isUPtr %}
            {{ propertySignature }}& {{ property.name | str }}Item = {{ arrayPropertyPath }}.Emplace_GetRef(NewObject<{{ property.type | str }}>());
            {%- else %}
            {{ propertySignature }}& {{ property.name | str }}Item = {{ arrayPropertyPath }}.Emplace_GetRef();
            {%- endif %}
            {%- assign propertyPath = property.name | str | append: 'Item' %}
            {%- if property.type.jsonType == "ToFromJsonObject" -%}
            if({{ jsonObject }}->TryGetObject(TempObject))
            {
                {{ propertyPath }}.JsonObject = *TempObject;
            }
            {%- elsif property.type.jsonType == "ToFromHandler" %}
            {%- if property.type.declarationType == "Enum" %}
            UE::{{ property.namespace }}::FromJson({{ jsonObject }}, {{ propertyPath }});
            {%- else %}
            {%- if isUPtr %}
            {{ propertyPath }}->FromJson({{ jsonObject }});
            {%- else %}
            {{ propertyPath }}.FromJson({{ jsonObject }});
            {%- endif %}
            {%- endif %}
            {%- else %}
            {%- elsif property.type.name == "DateTime" %}
                FString {{ propertyPath }}JsonString;
                if({{ jsonObject }}->TryGetString({{ propertyPath }}JsonString))
                {
                    FDateTime::ParseIso8601(*{{ propertyPath }}JsonString, {{ propertyPath }});
                }
            {%- else %}
                {{ jsonObject }}->TryGet{{ property.type | jsontype }}({{ propertyPath }});
            {%- endif -%}
        }