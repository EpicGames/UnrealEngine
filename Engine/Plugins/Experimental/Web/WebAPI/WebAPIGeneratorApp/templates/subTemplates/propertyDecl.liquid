    {%- assign propertySignature = propertyDecl.type | str -%}
    {%- if propertyDecl.type.prefix == 'U' -%}
    {%- assign propertySignature = propertySignature | prepend: "TObjectPtr<" | append: ">" -%}
    {%- endif %}
    {%- capture propertyDefaultValue -%}
    {{ propertyDecl | defaultvalue }}
    {%- endcapture -%}
    /** {{ propertyDecl.description }} */
    UPROPERTY(
    {{- propertyDecl.specifiers | kvp: "", ", " -}}
    {{- propertyDecl.metadata | kvp: "meta=(", ")" -}})
    {% if propertyDecl.isArray -%}
    TArray<{{ propertySignature }}> {{ propertyDecl.name | str }}
    {%- else -%}
    {{ propertySignature }} {{ propertyDecl.name | str }}
    {%- endif -%}
    {%- if propertyDefaultValue -%}
    {{- propertyDefaultValue | prepend: " = " -}}
    {%- endif -%}
    ;

