        {%- assign isArray = property.isArray -%}
        {%- assign isUPtr = property.type.prefix == 'U' -%}
        {%- assign jsonObject = 'JsonObject->Get()' -%}

        {%- assign propertyPath = property.name | str %}
        {%- if property.type.jsonPropertyToSerialize != '' %}
        {%- assign propertyPath = property.name | str | append: '.' | append: property.type.jsonPropertyToSerialize %}
        {%- endif %}

        {%- if isArray %}
        {%- assign arrayPropertyPath = propertyPath %}
        const TArray<TSharedPtr<FJsonValue>>* {{ property.name | str }}JsonArray = nullptr;
        if(JsonObject->Get()->TryGetArrayField(TEXT("{{ property.name | jsonname }}"), {{ property.name | str }}JsonArray))
        {
            {%- assign jsonArray = property.name | str | append: "JsonArray" %}
            {% render "subTemplates/propertyArrayFromJson", property: property, jsonArray: jsonArray %}
        }
        {%- else %}
        {%- if property.type.jsonType == "ToFromJsonObject" -%}
        {%- if isRoot -%}
        {{ propertyPath }}.JsonObject = *JsonObject;
        {%- else -%}
        if({{ jsonObject }}->TryGetObjectField(TEXT("{{ property.name | jsonname }}"), TempObject))
        {
            {{ propertyPath }}.JsonObject = *TempObject;
        }
        {%- endif -%}
        {%- elsif property.type.jsonType == "ToFromHandler" %}
            if(const TSharedPtr<FJsonValue> {{ propertyPath }}Json = {{ jsonObject }}->TryGetField(TEXT("{{ property.name | jsonname }}")))
            {
                {%- if property.type.declarationType == "Enum" %}
                UE::{{ property.namespace }}::FromJson({{ propertyPath }}Json, {{ propertyPath }});
                {%- else %}
                {%- if isUPtr %}
                if(!{{ propertyPath }})
                {
                    {{ propertyPath }} = NewObject<{{ property.type | str }}>();
                }
                {{ propertyPath }}->FromJson({{ propertyPath }}Json);
                {%- else %}
                {{ propertyPath }}.FromJson({{ propertyPath }}Json);
                {%- endif %}
                {%- endif %}
            }
        {%- elsif property.type.name == "DateTime" %}
            FString {{ propertyPath }}JsonString;
            if({{ jsonObject }}->TryGetStringField(TEXT("{{ property.name | jsonname }}"), {{ propertyPath }}JsonString))
            {
                FDateTime::ParseIso8601(*{{ propertyPath }}JsonString, {{ propertyPath }});
            }
        {%- else %}
            {{ jsonObject }}->TryGet{{ property.type | jsontype }}Field(TEXT("{{ property.name | jsonname }}"), {{ propertyPath }});
        {%- endif -%}

        {%- endif %}

