        {%- assign isArray = property.isArray -%}
        {%- assign isUPtr = property.type.prefix == 'U' -%}
        {%- assign propertySignature = property.type | str -%}
        {%- assign hasReturned = false -%}
        {%- if isUPtr -%}
        {%- assign propertySignature = propertySignature | prepend: "TObjectPtr<" | append: ">" -%}
        {%- endif %}
        {%- assign jsonObject = 'JsonObject' -%}

        {%- assign propertyPath = property.name | str %}
        {%- if property.type.jsonPropertyToSerialize.length > 0 %}
        {%- assign propertyPath = property.name | str | append: '.' | append: property.type.jsonPropertyToSerialize %}
        {%- endif %}

        {%- if isArray %}
        {%- assign arrayPropertyPath = propertyPath %}
        TArray<TSharedPtr<FJsonValue>> {{ property.name | str }}JsonArray;
        {%- if isRoot -%}
        {%- assign jsonObject = property.name | str | append: 'JsonArray' -%}
        {%- endif %}
        {{ property.name | str }}JsonArray.Reserve({{ arrayPropertyPath }}.Num());
        for(const {{ propertySignature }}& {{ property.name | str }}Item : {{ arrayPropertyPath }})
        {
            {%- assign jsonItemObject = property.name | str | append: 'JsonItem' %}

            {%- assign propertyPath = property.name | str | append: 'Item' %}
            {%- if property.type.jsonPropertyToSerialize.length > 0 %}
            {%- if isUPtr %}
            {%- assign propertyPath = property.name | str | append: 'Item' | append: "->" | append: property.type.jsonPropertyToSerialize %}
            {%- else %}
            {%- assign propertyPath = property.name | str | append: 'Item' | append: '.' | append: property.type.jsonPropertyToSerialize %}
            {%- endif -%}
            {%- endif -%}

            {%- capture jsonObjectDeclaration -%}
            TSharedPtr<FJsonValue{{ property.type | jsontype }}> {{ jsonItemObject }}
            {%- endcapture -%}

            {%- if property.type.jsonType == "ToFromJsonObject" -%}
            if({{ propertyPath }}.JsonObject.IsValid())
            {
                {{ jsonObjectDeclaration }} = MakeShared<FJsonValueObject>({{ propertyPath }}.JsonObject);
            {%- elsif property.type.jsonType == "ToFromHandler" %}
            {%- if property.type.declarationType == "Enum" %}
                {{ jsonObjectDeclaration }} = UE::{{ property.type.namespace }}::ToJson({{ propertyPath }});
            {%- else %}
            {%- if isUPtr %}
            {{ jsonObjectDeclaration }};
            if({{ propertyPath }})
            {
                {{ jsonItemObject }} = {{ propertyPath }}->ToJson();
            }
            {%- else %}
            {{ jsonObjectDeclaration }} = {{ propertyPath }}.ToJson();
            {%- endif %}
            {%- endif %}
            {%- elsif property.type.name == "DateTime" %}
                {{ jsonObjectDeclaration }} = MakeShared<FJsonValue{{ property.type | jsontype }}>({{ propertyPath }}.ToIso8601());
            {%- else %}
                {{ jsonObjectDeclaration }} = MakeShared<FJsonValue{{ property.type | jsontype }}>({{ propertyPath }});
            {%- endif %}
            {{ property.name | str }}JsonArray.Add({{ jsonItemObject }});
            {%- if property.type.jsonType == "ToFromJsonObject" -%}
            }
            {%- endif %}
        }
        {%- if isRoot -%}
        return MakeShared<FJsonValueArray>({{ jsonObject }});
        {%- assign hasReturned = true -%}
        {%- else %}
        {{ jsonObject }}->SetArrayField(TEXT("{{ property.name | jsonname }}"), {{ property.name | str }}JsonArray);
        {%- endif %}
        {%- else %}

        {%- if isRoot -%}
        {%- assign jsonObject = property.name | str | append: 'Json' -%}
        {%- endif %}

        {%- if property.type.jsonType == "ToFromJsonObject" -%}
        if({{ propertyPath }}.JsonObject.IsValid())
        {
            {%- if isRoot %}
            {%- unless hasReturned -%}
            return MakeShared<FJsonValueObject>({{ propertyPath }}.JsonObject);
            {%- assign hasReturned = true -%}
            {%- endunless %}
            {%- else -%}
            {{ jsonObject }}->SetObjectField(TEXT("{{ property.name | jsonname }}"), {{ propertyPath }}.JsonObject);
            {%- endif %}
        }
        {%- if isRoot %}
        return nullptr;
        {%- endif -%}
        {%- else -%}

        {%- if property.type.jsonType == "ToFromHandler" %}
        {%- if property.type.declarationType == "Enum" %}
            {%- if isRoot %}
            {%- unless hasReturned -%}
            return UE::{{ property.type.namespace }}::ToJson({{ propertyPath }});
            {%- assign hasReturned = true -%}
            {%- endunless %}
            {%- else -%}
            {{ jsonObject }}->SetField(TEXT("{{ property.name | jsonname }}"), UE::{{ property.type.namespace }}::ToJson({{ propertyPath }}));
            {%- endif %}
        {%- else %}
        {%- if isUPtr %}
        if({{ propertyPath }})
        {
            {%- if isRoot %}
            {%- unless hasReturned -%}
            return {{ propertyPath }}->ToJson();
            {%- assign hasReturned = true -%}
            {%- endunless %}
            {%- else -%}
            {{ jsonObject }}->SetField(TEXT("{{ property.name | jsonname }}"), {{ propertyPath }}->ToJson());
            {%- endif %}
        }
        {%- if isRoot %}
        return nullptr;
        {%- endif -%}
        {%- else %}
        {%- if isRoot %}
        {%- unless hasReturned -%}
        return {{ propertyPath }}.ToJson();
        {%- assign hasReturned = true -%}
        {%- endunless %}
        {%- else -%}
        {{ jsonObject }}->SetField(TEXT("{{ property.name | jsonname }}"), {{ propertyPath }}.ToJson());
        {%- endif %}
        {%- endif %}
        {%- endif %}
        {%- elsif property.type.name == "DateTime" %}
            {%- if isRoot %}
            {%- unless hasReturned -%}
            return MakeShared<FJsonValue{{ property.type | jsontype }}>({{ propertyPath }}.ToIso8601());
            {%- assign hasReturned = true -%}
            {%- endunless %}
            {%- else -%}
            {{ jsonObject }}->Set{{ property.type | jsontype }}Field(TEXT("{{ property.name | jsonname }}"), {{ propertyPath }}.ToIso8601());
            {%- endif %}
        {%- else %}
            {%- if isRoot %}
            {%- unless hasReturned -%}
            return MakeShared<FJsonValue{{ property.type | jsontype }}>({{ propertyPath }});
            {%- assign hasReturned = true -%}
            {%- endunless %}
            {%- else -%}
            {{ jsonObject }}->Set{{ property.type | jsontype }}Field(TEXT("{{ property.name | jsonname }}"), {{ propertyPath }});
            {%- endif %}
        {%- endif -%}

        {%- if isRoot -%}
        {%- unless hasReturned -%}
        return MakeShared<FJsonValue{{ property.type | jsontype }}>({{ jsonObject }});
        {%- assign hasReturned = true -%}
        {%- endunless %}
        {%- endif %}

        {%- endif -%}
        {%- endif -%}
