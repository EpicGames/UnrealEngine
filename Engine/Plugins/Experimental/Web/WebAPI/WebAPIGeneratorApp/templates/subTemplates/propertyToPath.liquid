
{%- assign propertyPath = property.name | str %}
{%- if property.type.jsonPropertyToSerialize.length > 0 %}
{%- assign propertyPath = property.name | str | append: '.' | append: property.type.jsonPropertyToSerialize %}
{%- endif -%}

    Args.Add(TEXT("{{ property.name | jsonname }}"), {{ propertyPath }});
