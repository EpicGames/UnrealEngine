{%- assign propertyPath = property.name | str -%}
{%- unless property.isRequired -%}
{%- if property.isArray %}
if (!{{ propertyPath }}.IsEmpty())
{%- else %}
{%- if property.type.jsonType == "ToFromJsonObject" %}
if ({{ propertyPath }}.JsonObject.IsValid())
{%- else %}
if ({{ propertyPath }} != {{ property | defaultvalue }})
{%- endif %}
{%- endif %}
{
{%- endunless -%}

{%- if property.type.jsonPropertyToSerialize.length > 0 -%}
{%- assign propertyPath = property.name | str | append: '.' | append: property.type.jsonPropertyToSerialize -%}
{%- endif -%}

{%- assign printFormatSpecifier = property.type.printFormatSpecifier -%}

{%- assign printFormatPrefix = "*" -%}
{%- assign printFormatSuffix = "" -%}
{%- if printFormatSpecifier != "s" -%}
{%- assign printFormatPrefix = "" -%}
{%- endif -%}

{%- assign typeStr = property.type | str -%}
{%- if typeStr == "bool" -%}
{%- assign printFormatPrefix = "" -%}
{%- assign printFormatSpecifier = "s" -%}
{%- assign printFormatSuffix = " ? TEXT(\"True\") : TEXT(\"False\")" -%}
{%- endif -%}

{%- assign printFormatExpression = property.type.printFormatExpression | replace: "{Property}", propertyPath | replace: "{Namespace}", property.type.namespace %}
{%- if property.isArray -%}
{%- capture printFormatExpression -%}
	FString::JoinBy({{ printFormatExpression }}, TEXT(","), [&](const {{ typeStr }}& InValue)
	{
		return {{ property.type.printFormatExpression | replace: "{Property}", "InValue" | replace: "{Namespace}", property.type.namespace }};
	})
{%- endcapture -%}
{%- endif %}

{%- if property.type.jsonType == "ToFromJsonObject" %}
	for(const TPair<FString, TSharedPtr<FJsonValue>>& KVP : {{ printFormatPrefix }}{{ printFormatExpression }}{{ printFormatSuffix }}.JsonObject->Values)
	{
		Result.Add(FString::Printf(TEXT("%s=%s"), *KVP.Key, *KVP.Value->AsString()));
	}
{%- else %}
    Result.Add(FString::Printf(TEXT("{{ property.name | jsonname }}=%{{ printFormatSpecifier }}"), {{ printFormatPrefix }}{{ printFormatExpression }}{{ printFormatSuffix }}));
{%- endif %}

{%- unless property.isRequired %}
}
{%- endunless -%}
