    /** Converts and populates object properties from Json. */
    void FromJson(const TSharedPtr<FJsonValue>& InJson);