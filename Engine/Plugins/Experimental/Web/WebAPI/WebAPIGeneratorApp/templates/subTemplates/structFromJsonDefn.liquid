void {{ structDefn.name | str }}::FromJson(const TSharedPtr<FJsonValue>& InJson)
{
    {%- assign propertiesWithoutStorageSpecified = structDefn.properties | where: "storage", "", "===", "" -%}
    {%- assign candidateProperties = structDefn.properties | where: "storage", "Body" | concat: propertiesWithoutStorageSpecified -%}

    {%- assign isRoot = false -%}
    {%- assign hasSingleCandidateProperty = candidateProperties.length == 1 -%}
    {%- if hasSingleCandidateProperty -%}
    {%- assign jsonTypeStr = candidateProperties[0].type | jsontype -%}
    {%- assign isRoot = candidateProperties[0].isMixin or jsonTypeStr == "object" or jsonTypeStr == "" or candidateProperties[0].isArray -%}
    {%- endif -%}

    {% comment -%} If only a single property, and its an array, the entire json response should be treated as an array{%- endcomment -%}
    {%- if candidateProperties.length == 1 and candidateProperties[0].isArray %}
	const TArray<TSharedPtr<FJsonValue>>* JsonValues = nullptr;
	if(InJson->TryGetArray(JsonValues))
	{
        {%- render "subTemplates/propertyArrayFromJson", property: candidateProperties[0], jsonArray: "JsonValues" %}
    {%- else %}
	TSharedPtr<FJsonObject>* JsonObject = nullptr;
	if(InJson->TryGetObject(JsonObject))
	{
        {%- assign hasJsonObjectProperty = structDefn.properties | where: "type.jsonType", "ToFromJsonObject" -%}
        {%- assign hasJsonObjectProperty = hasJsonObjectProperty.length > 0 -%}
        {%- unless isRoot -%}
        {%- if hasJsonObjectProperty -%}
        const TSharedPtr<FJsonObject>* TempObject = nullptr;
        {%- endif -%}
        {%- endunless -%}
        {%- for property in candidateProperties -%}
        {%- assign hasToFromJson = property.type.jsonType == "ToFromHandler" -%}
        {%- if property.isMixin and hasToFromJson -%}
        {{ property.name | str }}.FromJson(InJson);
        {%- else -%}
        {%- render "subTemplates/propertyFromJson", property: property, isRoot: isRoot %}
        {%- endif -%}
        {%- endfor -%}
    {%- endif -%}
	}
}