    /** Converts this object to Json. */
    TSharedPtr<FJsonValue> ToJson() const;

