{%- assign candidateProperties = structDefn.properties | where: "storage", "Body" -%}
{%- assign hasSingleCandidateProperty = candidateProperties.length == 1 -%}
TSharedPtr<FJsonValue> {{ structDefn.name | str }}::ToJson() const
{
    {%- assign isRoot = false -%}
    {%- if hasSingleCandidateProperty -%}
    {%- assign jsonTypeStr = candidateProperties[0].type | jsontype -%}
    {%- assign isRoot = candidateProperties[0].isMixin or jsonTypeStr == "object" or jsonTypeStr == "" or candidateProperties[0].isArray  -%}
    {%- endif -%}
    {%- unless isRoot -%}
    const TSharedPtr<FJsonObject> JsonObject = MakeShared<FJsonObject>();
    {%- endunless -%}

    {%- render "subTemplates/propertyToJson" for candidateProperties as property, isRoot: isRoot %}

    {%- unless isRoot -%}
    return MakeShared<FJsonValueObject>(JsonObject);
    {%- endunless -%}
}
