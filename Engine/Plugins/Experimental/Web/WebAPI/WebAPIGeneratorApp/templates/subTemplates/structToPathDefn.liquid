FString {{ structDefn.name | str }}::ToPath(const FString& InTemplatedPath) const
{
    FStringFormatNamedArguments Args;

    {%- assign candidateProperties = structDefn.properties | where: "storage", "Path" -%}
    {%- render "subTemplates/propertyToPath" for candidateProperties as property %}

    return FString::Format(*InTemplatedPath, Args);
}