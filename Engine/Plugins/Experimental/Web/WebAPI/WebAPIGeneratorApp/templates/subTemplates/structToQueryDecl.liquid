    /** Converts this object to a Url encoded query string. */
    FString ToQuery() const;