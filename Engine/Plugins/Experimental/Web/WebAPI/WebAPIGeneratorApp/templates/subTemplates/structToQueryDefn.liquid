FString {{ structDefn.name | str }}::ToQuery() const
{
    TArray<FString> Result;
    {%- assign candidateProperties = structDefn.properties | where: "storage", "Query" -%}
    {%- render "subTemplates/propertyToQuery" for candidateProperties as property %}

    if(Result.IsEmpty())
    {
        return TEXT("");
    }

    return FString::Join(Result, TEXT("&"));
}
