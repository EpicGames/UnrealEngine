{
  "compilerOptions": {
    "module": "commonjs",
    "resolveJsonModule": true,
    "esModuleInterop": true,
    "target": "es2018",
    "noImplicitAny": true,
    "moduleResolution": "node",
    "sourceMap": false,
    "outDir": "./build",
    "experimentalDecorators": true,
    "emitDecoratorMetadata": true,
    "strict": true,
  }
}
