### Unreal Engine 4 Networking over WebSockets Plugin

- Provides websocket transport layer for Unreal Engine 4.
- Uses [libwebsockets](http://libwebsockets.org) for the server side and client side for non HTML5 clients.
- HTML5 clients use emscripten's sockets abstraction.

#### How to

- Clone this as Engine/Platforms/Plugins/WebSocketNetworking.
- Run GenerateProjectFiles.bat (or ./GenerateProjectFiles.sh)
- Add the following section in `BaseEngine.ini`
```
[/Script/WebSocketNetworking.WebSocketNetDriver]
AllowPeerConnections=False
AllowPeerVoice=False
ConnectionTimeout=60.0
InitialConnectTimeout=120.0
RecentlyDisconnectedTrackingTime=180
AckTimeout=10.0
KeepAliveTime=20.2
MaxClientRate=15000
MaxInternetClientRate=10000
RelevantTimeout=5.0
SpawnPrioritySeconds=1.0
ServerTravelPause=4.0
NetServerMaxTickRate=30
LanServerMaxTickRate=35
WebSocketPort=8889
NetConnectionClassName="/Script/WebSocketNetworking.WebSocketConnection"
MaxPortCountToTry=512
```
In section [/Script/Engine.Engine] disable (by commenting out) ALL existing NetDriverDefinitions entries and then add:
```
NetDriverDefinitions=(DefName="GameNetDriver",DriverClassName="/Script/WebSocketNetworking.WebSocketNetDriver",DriverClassNameFallback="/Script/WebSocketsNetworking.IpNetDriver")
```
To enable this Net Driver.

Build! and follow existing Unreal Networking documentation to setup servers/clients.

#### Issues/Todo

Disconnect events on client or server side are not handled properly yet

Copyright Epic Games.
